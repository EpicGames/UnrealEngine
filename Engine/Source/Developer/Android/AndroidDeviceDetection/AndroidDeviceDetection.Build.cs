// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AndroidDeviceDetection : ModuleRules
{
	public AndroidDeviceDetection( ReadOnlyTargetRules Target ) : base(Target)
	{
		bRequiresPlatformSDK = true;

		BinariesSubFolder = "Android";

        PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Json",
				"JsonUtilities",
				"PIEPreviewDeviceSpecification",
				"Zen"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"TcpMessaging",
			}
		);

        if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
		}
		
		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DesktopPlatform",
					"SlateCore",
					"Slate",
				}
			);
		}

		DynamicallyLoadedModuleNames.AddRange(
            new string[]
            {
                "TcpMessaging"
            }
        );
    }
}
