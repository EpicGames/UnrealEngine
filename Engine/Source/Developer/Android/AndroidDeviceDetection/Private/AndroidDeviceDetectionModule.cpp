// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidDeviceDetectionModule.cpp: Implements the FAndroidDeviceDetectionModule class.
=============================================================================*/

#include "CoreTypes.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include "Async/EventCount.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/StringConv.h"
#include "Containers/Map.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "ITcpMessagingModule.h"
#include "Experimental/ZenServerInterface.h"

#include "PIEPreviewDeviceSpecification.h"
#include "JsonObjectConverter.h"
#include "Dom/JsonObject.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Internationalization/Regex.h"

#if WITH_EDITOR
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#endif
#include "String/ParseLines.h"

#define LOCTEXT_NAMESPACE "FAndroidDeviceDetectionModule" 

DEFINE_LOG_CATEGORY_STATIC(AndroidDeviceDetectionLog, Log, All);

static int32 GAndroidDeviceDetectionPollInterval = 10;
static FAutoConsoleVariableRef CVarAndroidDeviceDetectionPollInterval(
	TEXT("Android.DeviceDetectionPollInterval"),
	GAndroidDeviceDetectionPollInterval,
	TEXT("The number of seconds between polling for connected Android devices.\n")
	TEXT("Default: 10"),
	ECVF_Default
);

class FAndroidDeviceDetectionRunnable : public FRunnable
{
public:
	FAndroidDeviceDetectionRunnable(TMap<FString,FAndroidDeviceInfo>& InDeviceMap, FCriticalSection* InDeviceMapLock, FCriticalSection* InADBPathCheckLock) :
		DeviceMap(InDeviceMap),
		DeviceMapLock(InDeviceMapLock),
		ADBPathCheckLock(InADBPathCheckLock),
		bHasADBPath(false),
		bForceCheck(false)
	{
		TcpMessagingModule = FModuleManager::LoadModulePtr<ITcpMessagingModule>("TcpMessaging");
	}

public:

	// FRunnable interface.
	virtual bool Init(void) 
	{ 
		return true; 
	}

	virtual void Exit(void) 
	{
	}

	virtual void Stop()
	{
		bStopRequested.store(true, std::memory_order_relaxed);
		StopEvent.Notify();
	}

	virtual uint32 Run()
	{
		if (bStopRequested.load(std::memory_order_relaxed))
		{
			return 0;
		}

		for (int32 LoopCount = 10;;)
		{
			// query when we have waited 'GAndroidDeviceDetectionPollInterval' seconds.
			if (LoopCount++ >= GAndroidDeviceDetectionPollInterval || bForceCheck.load(std::memory_order_relaxed))
			{
				// Make sure we have an ADB path before checking
				FScopeLock PathLock(ADBPathCheckLock);
				if (bHasADBPath)
				{
					QueryConnectedDevices();
				}

				LoopCount = 0;
				bForceCheck = false;
			}

			{
				UE::FEventCountToken Token = StopEvent.PrepareWait();
				if (bStopRequested.load(std::memory_order_relaxed) || StopEvent.WaitFor(Token, UE::FMonotonicTimeSpan::FromSeconds(1.0)))
				{
					break;
				}
			}
		}

		return 0;
	}

	void UpdatePaths(FString InADBPath, FString InAvdHomePath, FString InGetPropCommand, bool InbGetExtensionsViaSurfaceFlinger)
	{
		ADBPath = MoveTemp(InADBPath);
		AvdHomePath = MoveTemp(InAvdHomePath);
		GetPropCommand = MoveTemp(InGetPropCommand);
		bGetExtensionsViaSurfaceFlinger = InbGetExtensionsViaSurfaceFlinger;

		bHasADBPath = !ADBPath.IsEmpty();
		// Force a check next time we go around otherwise it can take over 10sec to find devices
		bForceCheck = bHasADBPath;

		// If we have no path then clean the existing devices out
		if (!bHasADBPath && DeviceMap.Num() > 0)
		{
			DeviceMap.Reset();
		}
	}

private:
	bool ExecuteAdbCommand(const FString& CommandLine, FString* OutStdOut) const
	{
		if (bStopRequested.load(std::memory_order_relaxed) || !FPaths::FileExists(ADBPath))
		{
			return false;
		}

		UE::HAL::FInputPipe InputPipe = UE::HAL::NewPipe;
		if (!InputPipe)
		{
			return false;
		}
		
		UE::HAL::FProcess Process{{.Uri = *ADBPath, .Arguments = *CommandLine, .bDetached = true, .bHidden = true, .StdOut = InputPipe}};
		if (!Process)
		{
			return false;
		}
		
		int32 ExitCode = -1;
		
		for (UE::FMonotonicTimePoint MaxTime = UE::FMonotonicTimePoint::Now() + UE::FMonotonicTimeSpan::FromSeconds(10.0);;)
		{
			bool bIsRunning = Process.IsRunning();
			
			if (OutStdOut)
			{
				*OutStdOut += InputPipe.Read();
			}

			if (!bIsRunning)
			{
				ExitCode = *Process.GetExitCode();
				break;
			}

			FPlatformProcess::YieldThread();

			if (bStopRequested.load(std::memory_order_relaxed) || UE::FMonotonicTimePoint::Now() >= MaxTime)
			{
				Process.Kill();
				break;
			}
		}

		if (ExitCode != 0)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("The Android SDK command '%s' failed to run. Exit Code: %d\n"), *CommandLine, ExitCode);
			return false;
		}

		return true;
	}

	// searches for 'DPIString' and 
	int32 ExtractDPI(const FString& SurfaceFlingerOutput, const FString& DPIString)
	{
		int32 FoundDpi = INDEX_NONE;

		int32 DpiIndex = SurfaceFlingerOutput.Find(DPIString);
		if (DpiIndex != INDEX_NONE)
		{
			int32 StartIndex = INDEX_NONE;
			for (int32 i = DpiIndex; i < SurfaceFlingerOutput.Len(); ++i)
			{
				// if we somehow hit a line break character something went wrong and no digits were found on this line
				// we don't want to search the SurfaceFlinger feed so exit now
				if (FChar::IsLinebreak(SurfaceFlingerOutput[i]))
				{
					break;
				}

				// search for the first digit aka the beginning of the DPI value
				if (StartIndex == INDEX_NONE && FChar::IsDigit(SurfaceFlingerOutput[i]))
				{
					StartIndex = i;
				}
				// if we hit some non-numeric character extract the number and exit
				else if (StartIndex != INDEX_NONE && !FChar::IsDigit(SurfaceFlingerOutput[i]))
				{
					FString str = SurfaceFlingerOutput.Mid(StartIndex, i - StartIndex);
					FoundDpi = FCString::Atoi(*str);
					break;
				}
			}
		}

		return FoundDpi;
	}
	// retrieve the string between 'InOutStartIndex' and the start position of the next 'Token' substring
	// the white spaces of the resulting string are trimmed out at both ends 
	FString ExtractNextToken(int32& InOutStartIndex, const FString& SurfaceFlingerOutput, const FString& Token)
	{
		FString OutString;
		int32 StartIndex = InOutStartIndex;

		int32 EndIndex = SurfaceFlingerOutput.Find(Token, ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);

		if (EndIndex != INDEX_NONE)
		{
			InOutStartIndex = EndIndex + 1;
			// the index should point to the position before the token start
			--EndIndex;

			for (int32 i = StartIndex; i < EndIndex; ++i)
			{
				if (!FChar::IsWhitespace(SurfaceFlingerOutput[i]))
				{
					StartIndex = i;
					break;
				}
			}

			for (int32 i = EndIndex; i > StartIndex; --i)
			{
				if (!FChar::IsWhitespace(SurfaceFlingerOutput[i]))
				{
					EndIndex = i;
					break;
				}
			}

			OutString = SurfaceFlingerOutput.Mid(StartIndex, FMath::Max(0, EndIndex - StartIndex + 1));
		}

		return OutString;
	}

	void ExtractGPUInfo(FString& outGLVersion, FString& outGPUFamily, const FString& SurfaceFlingerOutput)
	{
		int32 FoundDpi = INDEX_NONE;

		int32 LineIndex = SurfaceFlingerOutput.Find(TEXT("GLES:"));
		if (LineIndex != INDEX_NONE)
		{
			int32 StartIndex = SurfaceFlingerOutput.Find(TEXT(":"), ESearchCase::IgnoreCase, ESearchDir::FromStart, LineIndex);
			if (StartIndex != INDEX_NONE)
			{
				++StartIndex;

				FString GPUVendorString = ExtractNextToken(StartIndex, SurfaceFlingerOutput, TEXT(","));
				outGPUFamily = ExtractNextToken(StartIndex, SurfaceFlingerOutput, TEXT(","));
				outGLVersion = ExtractNextToken(StartIndex, SurfaceFlingerOutput, TEXT("\n"));

				// ANGLE-based devices report the GL version as "ANGLE (<GPU Name>)"
				{
					FRegexPattern AnglePattern(TEXT("ANGLE\\s*\\((.+)\\)"));
					FRegexMatcher AngleMatcher(AnglePattern, outGLVersion);
					if (AngleMatcher.FindNext())
					{
						outGPUFamily = AngleMatcher.GetCaptureGroup(1);
					}
				}
			}
		}
	}

	void QueryConnectedDevices()
	{
		// grab the list of devices via adb
		FString StdOut;
		if (!ExecuteAdbCommand(TEXT("devices -l"), &StdOut))
		{
			return;
		}

		auto AbiToArchitecture = [](FStringView Abi) -> FStringView
		{
			if (Abi == TEXTVIEW("arm64-v8a"))
			{
				return TEXTVIEW("arm64");
			}
			else if (Abi == TEXTVIEW("armeabi-v7a"))
			{
				return TEXTVIEW("arm32");
			}
			else if (Abi == TEXTVIEW("x86_64"))
			{
				return TEXTVIEW("x64");
			}
			else if (Abi == TEXTVIEW("x86"))
			{
				return TEXTVIEW("x86");
			}
			else
			{
				return {};
			}
		};

		// separate out each line
		TArray<FString> DeviceStrings;
		StdOut = StdOut.Replace(TEXT("\r"), TEXT("\n"));
		StdOut.ParseIntoArray(DeviceStrings, TEXT("\n"), true);

		TArray<FString> AvdNames;
		IFileManager::Get().FindFiles(AvdNames, *AvdHomePath, TEXT("ini"));

		for (FString& AvdName : AvdNames)
		{
			AvdName.LeftChopInline(4);
		}

		// list of any existing port forwardings, filled in when we find a device we need to add.
		TArray<FString> PortForwardings;

		// a list containing all devices found this time, so we can remove anything not in this list
		TSet<FStringView> CurrentlyConnectedDevices;

		for (const FString& DeviceString : DeviceStrings)
		{
			if (bStopRequested.load(std::memory_order_relaxed))
			{
				return;
			}
			
			// skip over non-device lines
			if (DeviceString.StartsWith("* ") || DeviceString.StartsWith("List "))
			{
				continue;
			}

			int32 TabIndex;

			if (!DeviceString.FindChar(TCHAR(' '), TabIndex) && !DeviceString.FindChar(TCHAR('\t'), TabIndex))
			{
				continue;
			}

			FStringView Status = FStringView{DeviceString}.Mid(TabIndex + 1).TrimStart();

			if (int32 TabIndex2; Status.FindChar(TCHAR(' '), TabIndex2) || Status.FindChar(TCHAR('\t'), TabIndex2))
			{
				Status.LeftInline(TabIndex2);
			}

			bool bAuthorized = Status != TEXT("unauthorized");

			if (bAuthorized && Status != TEXT("device"))
			{
				continue;
			}

			FString SerialNumber = DeviceString.Left(TabIndex), DeviceId, AvdName;

			// Find the AVD name
			if (FString AvdNameOutput; SerialNumber.StartsWith("emulator-") && ExecuteAdbCommand(*(TEXT("-s ") + SerialNumber + TEXT(" emu avd name")), &AvdNameOutput))
			{
				const TCHAR* Ptr = *AvdNameOutput;
				FParse::Line(&Ptr, AvdName);
				DeviceId = TEXT("avd-") + AvdName;
				
				AvdNames.RemoveSingle(AvdName);
			}
			else
			{
				DeviceId = SerialNumber;
			}
			
			FAndroidDeviceInfo* Device = DeviceMap.Find(DeviceId);
			
			if (!Device || Device->bAuthorizedDevice != bAuthorized || Device->SerialNumber != SerialNumber)
			{
				FAndroidDeviceInfo NewDeviceInfo;

				if (!bAuthorized)
				{
					//note: AndroidTargetDevice::GetName() does not fetch this value, do not rely on this
					NewDeviceInfo.DeviceName = TEXT("Unauthorized - enable USB debugging");
				}
				else
				{
					// grab the Android version
					const FString AndroidVersionCommand = FString::Printf(TEXT("-s %s %s ro.build.version.release"), *SerialNumber, *GetPropCommand);
					if (!ExecuteAdbCommand(*AndroidVersionCommand, &NewDeviceInfo.HumanAndroidVersion))
					{
						continue;
					}
					NewDeviceInfo.HumanAndroidVersion = NewDeviceInfo.HumanAndroidVersion.Replace(TEXT("\r"), TEXT("")).Replace(TEXT("\n"), TEXT(""));
					NewDeviceInfo.HumanAndroidVersion.TrimStartAndEndInline();

					// grab the Android SDK version
					const FString SDKVersionCommand = FString::Printf(TEXT("-s %s %s ro.build.version.sdk"), *SerialNumber, *GetPropCommand);
					FString SDKVersionString;
					if (!ExecuteAdbCommand(*SDKVersionCommand, &SDKVersionString))
					{
						continue;
					}
					NewDeviceInfo.SDKVersion = FCString::Atoi(*SDKVersionString);
					if (NewDeviceInfo.SDKVersion <= 0)
					{
						NewDeviceInfo.SDKVersion = INDEX_NONE;
					}

					if (FString AbiOutput; !ExecuteAdbCommand(*FString::Printf(TEXT("-s %s shell getprop ro.product.cpu.abi"), *SerialNumber), &AbiOutput))
					{
						continue;
					}
					else
					{
						NewDeviceInfo.Architecture = AbiToArchitecture(FStringView{AbiOutput}.TrimStartAndEnd());
					}

					if (bGetExtensionsViaSurfaceFlinger)
					{
						// get the GL extensions string (and a bunch of other stuff)
						const FString ExtensionsCommand = FString::Printf(TEXT("-s %s shell dumpsys SurfaceFlinger"), *SerialNumber);
						if (!ExecuteAdbCommand(*ExtensionsCommand, &NewDeviceInfo.GLESExtensions))
						{
							continue;
						}

						// extract DPI information
						int32 XDpi = ExtractDPI(NewDeviceInfo.GLESExtensions, TEXT("x-dpi"));
						int32 YDpi = ExtractDPI(NewDeviceInfo.GLESExtensions, TEXT("y-dpi"));

						if (XDpi != INDEX_NONE && YDpi != INDEX_NONE)
						{
							NewDeviceInfo.DeviceDPI = (XDpi + YDpi) / 2;
						}

						// extract OpenGL version and GPU family name
						ExtractGPUInfo(NewDeviceInfo.OpenGLVersionString, NewDeviceInfo.GPUFamilyString, NewDeviceInfo.GLESExtensions);
					}

					// grab device brand
					{
						FString ExecCommand = FString::Printf(TEXT("-s %s %s ro.product.brand"), *SerialNumber, *GetPropCommand);

						FString RoProductBrand;
						ExecuteAdbCommand(*ExecCommand, &RoProductBrand);
						const TCHAR* Ptr = *RoProductBrand;
						FParse::Line(&Ptr, NewDeviceInfo.DeviceBrand);
					}

					// grab screen resolution
					{
						FString ResolutionString;
						const FString ExecCommand = FString::Printf(TEXT("-s %s shell wm size"), *SerialNumber);
						if (ExecuteAdbCommand(*ExecCommand, &ResolutionString))
						{
							bool bFoundResX = false;
							int32 StartIndex = INDEX_NONE;
							for (int32 Index = 0; Index < ResolutionString.Len(); ++Index)
							{
								if (StartIndex == INDEX_NONE && FChar::IsDigit(ResolutionString[Index]))
								{
									StartIndex = Index;
								}
								else if (StartIndex != INDEX_NONE && !FChar::IsDigit(ResolutionString[Index]))
								{
									FString str = ResolutionString.Mid(StartIndex, Index - StartIndex);

									if (bFoundResX)
									{
										NewDeviceInfo.ResolutionY = FCString::Atoi(*str);
										break;
									}
									else
									{
										NewDeviceInfo.ResolutionX = FCString::Atoi(*str);
										bFoundResX = true;
										StartIndex = INDEX_NONE;
									}
								}
							}
						}
					}

					// grab the GL ES version
					FString GLESVersionString;
					const FString GLVersionCommand = FString::Printf(TEXT("-s %s %s ro.opengles.version"), *SerialNumber, *GetPropCommand);
					if (!ExecuteAdbCommand(*GLVersionCommand, &GLESVersionString))
					{
						continue;
					}
					NewDeviceInfo.GLESVersion = FCString::Atoi(*GLESVersionString);

					// Find the device model
					{
						FRegexPattern Pattern(TEXT("\\smodel:(.*?)(?=\\s+\\w+:|$)"));
						FRegexMatcher Matcher(Pattern, DeviceString);
						if (Matcher.FindNext())
						{
							NewDeviceInfo.Model = Matcher.GetCaptureGroup(1).TrimStartAndEnd();
						}
					}
					// find the product model (this must match java's android.os.build.model)
					FString ModelCommand = FString::Printf(TEXT("-s %s %s ro.product.model"), *SerialNumber, *GetPropCommand);
					FString RoProductModel;
					if (ExecuteAdbCommand(*ModelCommand, &RoProductModel) )
					{
						if(!RoProductModel.IsEmpty())
						{
							NewDeviceInfo.Model = RoProductModel.TrimStartAndEnd();
						}
					}
					
					// Find the build ID
					FString BuildNumberString;
					const FString BuildNumberCommand = FString::Printf(TEXT("-s %s %s ro.build.display.id"), *SerialNumber, *GetPropCommand);
					if (ExecuteAdbCommand(*BuildNumberCommand, &BuildNumberString))
					{
						NewDeviceInfo.BuildNumber = BuildNumberString.TrimStartAndEnd();
					}

					// Scan lines looking for ContainsTerm
					auto FindLineContaining = [](const FString& SourceString, const FString& ContainsTerm)
					{
						FString result;
						UE::String::ParseLines(SourceString,
							[&result, &ContainsTerm](const FStringView& Line)
							{
								if (result.IsEmpty() && Line.Contains(ContainsTerm))
								{
									result = Line;
								}
							});

						return result;
					};

					// Parse vulkan version:
					auto MajorVK = [](uint32 Version) { return (((uint32_t)(Version) >> 22) & 0x7FU); };
					auto MinorVK = [](uint32 Version) { return (((uint32_t)(Version) >> 12) & 0x3FFU); };
					auto PatchVK = [](uint32 Version) { return ((uint32_t)(Version) & 0xFFFU); };
					FString FeaturesString;
					const FString FeaturesStringCommand = FString::Printf(TEXT("-s %s shell pm list features"), *SerialNumber);
					if (ExecuteAdbCommand(*FeaturesStringCommand, &FeaturesString))
					{
						FString VulkanVersionLine = FindLineContaining(FeaturesString, TEXT("android.hardware.vulkan.version"));
						const FRegexPattern RegexPattern(TEXT("android\\.hardware\\.vulkan\\.version=(\\d*)"));
						FRegexMatcher RegexMatcher(RegexPattern, *VulkanVersionLine);
						if (RegexMatcher.FindNext())
						{
							uint32 PackedVersion = (uint32)FCString::Atoi64(*RegexMatcher.GetCaptureGroup(1));
							NewDeviceInfo.VulkanVersion = FString::Printf(TEXT("%d.%d.%d"), MajorVK(PackedVersion), MinorVK(PackedVersion), PatchVK(PackedVersion));
						}
					}

					// try vkjson:
					FString VKJsonString;
					const FString VKJsonStringCommand = FString::Printf(TEXT("-s %s shell cmd gpu vkjson"), *SerialNumber);
					if (ExecuteAdbCommand(*VKJsonStringCommand, &VKJsonString))
					{
						FString VulkanVersionLine = FindLineContaining(VKJsonString, TEXT("apiVersion"));

						const FRegexPattern RegexPattern(TEXT("\"apiVersion\"\\s*:\\s*(\\d*)"));
 						FRegexMatcher RegexMatcher(RegexPattern, *VulkanVersionLine);
 						if (RegexMatcher.FindNext())
						{
							FString VulkanVersion = RegexMatcher.GetCaptureGroup(1);
							uint32 PackedVersion = (uint32)FCString::Atoi64(*VulkanVersion);
							if(PackedVersion>0)
							{
								NewDeviceInfo.VulkanVersion = FString::Printf(TEXT("%d.%d.%d"), MajorVK(PackedVersion), MinorVK(PackedVersion), PatchVK(PackedVersion));
							}
						}
					}
					
					if (NewDeviceInfo.VulkanVersion.IsEmpty())
					{
						NewDeviceInfo.VulkanVersion = TEXT("0.0.0");
					}

					// create the hardware field
					FString HardwareCommand = FString::Printf(TEXT("-s %s %s ro.hardware"), *SerialNumber, *GetPropCommand);
					FString RoHardware;
					{
						ExecuteAdbCommand(*HardwareCommand, &RoHardware);
						const TCHAR* Ptr = *RoHardware;
						FParse::Line(&Ptr, NewDeviceInfo.Hardware);
					}
					if (RoHardware.Contains(TEXT("qcom")))
					{
						HardwareCommand = FString::Printf(TEXT("-s %s %s ro.hardware.chipname"), *SerialNumber, *GetPropCommand);
						ExecuteAdbCommand(*HardwareCommand, &RoHardware);
						const TCHAR* Ptr = *RoHardware;
						FParse::Line(&Ptr, NewDeviceInfo.Hardware);
					}
					{
						HardwareCommand = FString::Printf(TEXT("-s %s %s ro.soc.model"), *SerialNumber, *GetPropCommand);
						FString RoSOCModelIn;
						FString RoSOCModelOut;
						ExecuteAdbCommand(*HardwareCommand, &RoSOCModelIn);
						const TCHAR* Ptr = *RoSOCModelIn;
						FParse::Line(&Ptr, RoSOCModelOut);
						if (!RoSOCModelOut.IsEmpty())
						{
							NewDeviceInfo.Hardware = RoSOCModelOut;
						}
					}

					// Read hardware from cpuinfo:
					FString CPUInfoString;
					const FString CPUInfoCommand = FString::Printf(TEXT("-s %s shell cat /proc/cpuinfo"), *SerialNumber);
					if (ExecuteAdbCommand(*CPUInfoCommand, &CPUInfoString))
					{
						FString HardwareLine = FindLineContaining(CPUInfoString, TEXT("Hardware"));

						const FRegexPattern RegexPattern(TEXT("Hardware\\s*:\\s*(.*)"));
						FRegexMatcher RegexMatcher(RegexPattern, *HardwareLine);
						if (RegexMatcher.FindNext())
						{
							NewDeviceInfo.Hardware = RegexMatcher.GetCaptureGroup(1);
						}
					}

					// Total physical mem:
					FString MemTotalString;
					const FString MemTotalCommand = FString::Printf(TEXT("-s %s shell cat /proc/meminfo"), *SerialNumber);
					if (ExecuteAdbCommand(*MemTotalCommand, &MemTotalString))
					{
						FString MemTotalLine = FindLineContaining(MemTotalString, TEXT("MemTotal"));

						const FRegexPattern RegexPattern(TEXT("MemTotal:\\s*(\\d*)"));
						FRegexMatcher RegexMatcher(RegexPattern, *MemTotalLine);
						if (RegexMatcher.FindNext())
						{
							NewDeviceInfo.TotalPhysicalKB = (uint32)FCString::Atoi64(*RegexMatcher.GetCaptureGroup(1));
						}
					}

					// parse the device name
					{
						FRegexPattern Pattern(TEXT("\\sdevice:(.*?)(?=\\s+\\w+:|$)"));
						FRegexMatcher Matcher(Pattern, DeviceString);
						if (Matcher.FindNext())
						{
							NewDeviceInfo.DeviceName = Matcher.GetCaptureGroup(1).TrimStartAndEnd();
						}
					}
					
					if (NewDeviceInfo.DeviceName.IsEmpty())
					{
						FString DeviceCommand = FString::Printf(TEXT("-s %s %s ro.product.device"), *SerialNumber, *GetPropCommand);
						FString RoProductDevice;
						ExecuteAdbCommand(*DeviceCommand, &RoProductDevice);
						const TCHAR* Ptr = *RoProductDevice;
						FParse::Line(&Ptr, NewDeviceInfo.DeviceName);
					}

					// establish port forwarding if we're doing messaging
					if (TcpMessagingModule != nullptr)
					{
						// fill in the port forwarding array if needed
						if (PortForwardings.Num() == 0)
						{
							FString ForwardList;
							if (ExecuteAdbCommand(TEXT("forward --list"), &ForwardList))
							{
								ForwardList = ForwardList.Replace(TEXT("\r"), TEXT("\n"));
								ForwardList.ParseIntoArray(PortForwardings, TEXT("\n"), true);
							}
						}

						// check if this device already has port forwarding enabled for message bus, eg from another editor session
						for (FString& FwdString : PortForwardings)
						{
							const TCHAR* Ptr = *FwdString;
							FString FwdSerialNumber, FwdHostPortString, FwdDevicePortString;
							uint16 FwdHostPort, FwdDevicePort;
							if (FParse::Token(Ptr, FwdSerialNumber, false) && FwdSerialNumber == SerialNumber &&
								FParse::Token(Ptr, FwdHostPortString, false) && FParse::Value(*FwdHostPortString, TEXT("tcp:"), FwdHostPort) &&
								FParse::Token(Ptr, FwdDevicePortString, false) && FParse::Value(*FwdDevicePortString, TEXT("tcp:"), FwdDevicePort) && FwdDevicePort == 6666)
							{
								NewDeviceInfo.HostMessageBusPort = FwdHostPort;
								break;
							}
						}

						// if not, setup TCP port forwarding for message bus on first available TCP port above 6666
						if (NewDeviceInfo.HostMessageBusPort == 0)
						{
							uint16 HostMessageBusPort = 6666;
							bool bFoundPort;
							do
							{
								bFoundPort = true;
								for (auto It = DeviceMap.CreateConstIterator(); It; ++It)
								{
									if (HostMessageBusPort == It.Value().HostMessageBusPort)
									{
										bFoundPort = false;
										HostMessageBusPort++;
										break;
									}
								}
							} while (!bFoundPort);

							FString DeviceCommand = FString::Printf(TEXT("-s %s forward tcp:%d tcp:6666"), *SerialNumber, HostMessageBusPort);
							ExecuteAdbCommand(*DeviceCommand, nullptr);
							NewDeviceInfo.HostMessageBusPort = HostMessageBusPort;
						}

						TcpMessagingModule->AddOutgoingConnection(FString::Printf(TEXT("127.0.0.1:%d"), NewDeviceInfo.HostMessageBusPort));
					}

					// Add reverse port forwarding
					uint16 ReversePortMappings[]
					{
						41899,	// Network file server, DEFAULT_TCP_FILE_SERVING_PORT in NetworkMessage.h
						1981,	// Unreal Insights data collection, TraceInsightsModule.cpp
						UE::Zen::IsDefaultServicePresent() ? UE::Zen::GetDefaultServiceInstance().GetEndpoint().GetPort() : uint16(0), // Zen Store, usually defaults to 8558
						0 // end of list
					};

					for (int32 Idx=0; ReversePortMappings[Idx] > 0; Idx++)
					{
						FString DeviceCommand = FString::Printf(TEXT("-s %s reverse tcp:%d tcp:%d"), *SerialNumber, ReversePortMappings[Idx], ReversePortMappings[Idx]);
						// It doesn't really matter if a mapping already exists. There is no listening local port so no contention between multiple editor instances
						ExecuteAdbCommand(*DeviceCommand, nullptr);
					}

					FString WindowDisplaysOutput;
					const FString ExtensionsCommand = FString::Printf(TEXT("-s %s shell dumpsys window displays"), *SerialNumber);
					if (!ExecuteAdbCommand(*ExtensionsCommand, &WindowDisplaysOutput))
					{
						continue;
					}
					else
					{
						WindowDisplaysOutput = WindowDisplaysOutput.ToLower();
						const FRegexPattern RegexPattern(TEXT(".*initcutout.*insets=rect\\((\\d+)\\W*,\\s*(\\d+)\\W*-\\s*(\\d+)\\W*,\\s*(\\d+)\\W*\\)"));
						FRegexMatcher RegexMatcher(RegexPattern, *WindowDisplaysOutput);
						if (RegexMatcher.FindNext())
						{
							// store the insets independently from the resolution.
							NewDeviceInfo.InsetsLeft = FCString::Atof(*RegexMatcher.GetCaptureGroup(1)) / NewDeviceInfo.ResolutionX;
							NewDeviceInfo.InsetsTop = FCString::Atof(*RegexMatcher.GetCaptureGroup(2)) / NewDeviceInfo.ResolutionY;
							NewDeviceInfo.InsetsRight = FCString::Atof(*RegexMatcher.GetCaptureGroup(3)) / NewDeviceInfo.ResolutionX;
							NewDeviceInfo.InsetsBottom = FCString::Atof(*RegexMatcher.GetCaptureGroup(4)) / NewDeviceInfo.ResolutionY;
						}
					}
				}

				NewDeviceInfo.DeviceId = MoveTemp(DeviceId);
				NewDeviceInfo.AvdName = MoveTemp(AvdName);
				NewDeviceInfo.bAuthorizedDevice = bAuthorized;
				NewDeviceInfo.SerialNumber = MoveTemp(SerialNumber);

				// add the device to the map
				{
					FScopeLock ScopeLock(DeviceMapLock);

					if (!Device)
					{
						Device = &DeviceMap.Add(NewDeviceInfo.DeviceId);
					}

					*Device = MoveTemp(NewDeviceInfo);
				}
			}
			
			CurrentlyConnectedDevices.Add(Device->DeviceId);
		}

		for (FString& AvdName : AvdNames)
		{
			if (bStopRequested.load(std::memory_order_relaxed))
			{
				return;
			}

			FString DeviceId = TEXT("avd-") + AvdName;
			FAndroidDeviceInfo* Device = DeviceMap.Find(DeviceId);
			
			if (!Device || !Device->SerialNumber.IsEmpty())
			{
				FString Architecture;
				
				if (!FFileHelper::LoadFileToStringWithLineVisitor(*FPaths::Combine(AvdHomePath, AvdName + TEXT(".avd"), TEXT("config.ini")), [&](FStringView Line)
				{
					if (int32 Separator; Line.FindChar('=', Separator))
					{
						if (FStringView Key = Line.Left(Separator).TrimStartAndEnd(); Key == TEXT("hw.cpu.arch"))
						{
							Architecture = AbiToArchitecture(Line.RightChop(Separator + 1).TrimStartAndEnd());
						}
					}
				}))
				{
					continue;
				}

				FAndroidDeviceInfo NewDeviceInfo
				{
					.DeviceId = MoveTemp(DeviceId),
					.AvdName = MoveTemp(AvdName),
					.Architecture = MoveTemp(Architecture),
					.GLESExtensions = TEXT("GL_KHR_texture_compression_astc_ldr"),
					.GLESVersion = 0x30001,
					.bAuthorizedDevice = true,
					.VulkanVersion = TEXT("1.1.0")
				};
				
				{
					FScopeLock ScopeLock(DeviceMapLock);

					if (!Device)
					{
						Device = &DeviceMap.Add(NewDeviceInfo.DeviceId);
					}

					*Device = MoveTemp(NewDeviceInfo);
				}
			}
			
			CurrentlyConnectedDevices.Add(Device->DeviceId);
		}

		// loop through the previously connected devices list and remove any that aren't still connected from the updated DeviceMap
		TArray<FString> DevicesToRemove;

		for (const auto& Pair : DeviceMap)
		{
			if (!CurrentlyConnectedDevices.Contains(Pair.Key))
			{
				if (TcpMessagingModule && Pair.Value.HostMessageBusPort != 0)
				{
					TcpMessagingModule->RemoveOutgoingConnection(FString::Printf(TEXT("127.0.0.1:%d"), Pair.Value.HostMessageBusPort));
				}

				DevicesToRemove.Add(Pair.Key);
			}
		}

		{
			FScopeLock ScopeLock(DeviceMapLock);

			for (const FString& Device : DevicesToRemove)
			{
				DeviceMap.Remove(Device);
			}
		}
	}

private:
	mutable UE::FEventCount StopEvent;
	std::atomic_bool bStopRequested = false;

	// path to the adb command
	FString ADBPath;
	FString AvdHomePath;
	FString GetPropCommand;
	bool bGetExtensionsViaSurfaceFlinger;

	TMap<FString,FAndroidDeviceInfo>& DeviceMap;
	FCriticalSection* DeviceMapLock;

	FCriticalSection* ADBPathCheckLock;
	bool bHasADBPath;
	std::atomic_bool bForceCheck;

	ITcpMessagingModule* TcpMessagingModule;
};

class FAndroidDeviceDetection : public IAndroidDeviceDetection
{
public:

	FAndroidDeviceDetection() 
		: DetectionThread(nullptr)
		, DetectionThreadRunnable(nullptr)
	{
		// create and fire off our device detection thread
		DetectionThreadRunnable = new FAndroidDeviceDetectionRunnable(DeviceMap, &DeviceMapLock, &ADBPathCheckLock);
		DetectionThread = FRunnableThread::Create(DetectionThreadRunnable, TEXT("FAndroidDeviceDetectionRunnable"));
	}

	virtual ~FAndroidDeviceDetection()
	{
		if (DetectionThreadRunnable && DetectionThread)
		{
			DetectionThreadRunnable->Stop();
			DetectionThread->WaitForCompletion();
		}
	}

	virtual void Initialize(const TCHAR* InSDKDirectoryEnvVar, const TCHAR* InSDKRelativeExePath, const TCHAR* InGetPropCommand, bool InbGetExtensionsViaSurfaceFlinger) override
	{
		SDKDirEnvVar = InSDKDirectoryEnvVar;
		SDKRelativeExePath = InSDKRelativeExePath;
		GetPropCommand = InGetPropCommand;
		bGetExtensionsViaSurfaceFlinger = InbGetExtensionsViaSurfaceFlinger;
		UpdateADBPath();
	}

	virtual const TMap<FString,FAndroidDeviceInfo>& GetDeviceMap() override
	{
		return DeviceMap;
	}

	virtual FCriticalSection* GetDeviceMapLock() override
	{
		return &DeviceMapLock;
	}

	virtual FString GetADBPath() override
	{
		FScopeLock PathUpdateLock(&ADBPathCheckLock);
		return ADBPath;
	}

	virtual void UpdateADBPath() override
	{
		FScopeLock PathUpdateLock(&ADBPathCheckLock);

		FString AndroidHomeDirectory = FPlatformMisc::GetEnvironmentVariable(*SDKDirEnvVar);
		FString AndroidUserHomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("ANDROID_USER_HOME"));
		FString AndroidEmulatorHomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("ANDROID_EMULATOR_HOME"));
		FString AndroidAvdHomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("ANDROID_AVD_HOME"));

#if PLATFORM_MAC || PLATFORM_LINUX
		if (AndroidHomeDirectory.IsEmpty() || AndroidUserHomeDirectory.IsEmpty() || AndroidEmulatorHomeDirectory.IsEmpty() || AndroidAvdHomeDirectory.IsEmpty())
		{
#if PLATFORM_LINUX
			// didn't find ANDROID_HOME, so parse the .bashrc file on Linux
			FArchive* FileReader = IFileManager::Get().CreateFileReader(*FString("~/.bashrc"));
#else
			// didn't find ANDROID_HOME, so parse the .zshrc file on MAC
			FArchive* FileReader = IFileManager::Get().CreateFileReader(*FString([@"~/.zshrc" stringByExpandingTildeInPath]));
			if (!FileReader)
			{
				// Fallbacks
				FileReader = IFileManager::Get().CreateFileReader(*FString([@"~/.bashrc" stringByExpandingTildeInPath]));
				if (!FileReader)
				{
					FileReader = IFileManager::Get().CreateFileReader(*FString([@"~/.bash_profile" stringByExpandingTildeInPath]));
					UE_LOGF(LogCore, Warning, ".zshrc is missing. Falling back to .bash_profile.");
				}
				else
				{
					UE_LOGF(LogCore, Warning, ".zshrc is missing. Falling back to .bashrc.");
				}
			}
#endif
			if (FileReader)
			{
				const int64 FileSize = FileReader->TotalSize();
				ANSICHAR* AnsiContents = (ANSICHAR*)FMemory::Malloc(FileSize + 1);
				FileReader->Serialize(AnsiContents, FileSize);
				FileReader->Close();
				delete FileReader;

				AnsiContents[FileSize] = 0;
				TArray<FString> Lines;
				FString(ANSI_TO_TCHAR(AnsiContents)).ParseIntoArrayLines(Lines);
				FMemory::Free(AnsiContents);

				auto UpdateEnvironmentVariable = [](const FString& Line, const TCHAR* Key, FString& Value)
				{
					if (Value.IsEmpty() && Line.StartsWith(FString::Printf(TEXT("export %s="), Key)))
					{
						FString Directory;
						Line.Split(TEXT("="), NULL, &Directory);
						Directory.ReplaceInline(TEXT("\""), TEXT(""));
						Value = MoveTemp(Directory);
						setenv(TCHAR_TO_ANSI(Key), TCHAR_TO_ANSI(*Value), 1);
						return true;
					}

					return false;
				};

				for (int32 Index = Lines.Num()-1; Index >=0; Index--)
				{
					if (UpdateEnvironmentVariable(Lines[Index], *SDKDirEnvVar, AndroidHomeDirectory))
					{
						continue;
					}
					else if (UpdateEnvironmentVariable(Lines[Index], TEXT("ANDROID_USER_HOME"), AndroidUserHomeDirectory))
					{
						continue;
					}
					else if (UpdateEnvironmentVariable(Lines[Index], TEXT("ANDROID_EMULATOR_HOME"), AndroidEmulatorHomeDirectory))
					{
						continue;
					}
					else if (UpdateEnvironmentVariable(Lines[Index], TEXT("ANDROID_AVD_HOME"), AndroidAvdHomeDirectory))
					{
						continue;
					}
				}
			}
		}
#endif

		if (!AndroidHomeDirectory.IsEmpty())
		{
			ADBPath = FPaths::Combine(*AndroidHomeDirectory, SDKRelativeExePath);

			// if it doesn't exist then just clear the path as we might set it later
			if (!FPaths::FileExists(ADBPath))
			{
				ADBPath.Empty();
			}
		}
		else
		{
			ADBPath.Empty();
		}

		if (AndroidAvdHomeDirectory.IsEmpty())
		{
			if (AndroidEmulatorHomeDirectory.IsEmpty())
			{
				if (AndroidUserHomeDirectory.IsEmpty())
				{
					AndroidUserHomeDirectory = FPaths::Combine(FPlatformProcess::UserHomeDir(), TEXT(".android"));
				}
				
				AndroidEmulatorHomeDirectory = MoveTemp(AndroidUserHomeDirectory);
			}
			
			AndroidAvdHomeDirectory = FPaths::Combine(AndroidEmulatorHomeDirectory, TEXT("avd"));
		}

		AvdHomePath = MoveTemp(AndroidAvdHomeDirectory);
		
		DetectionThreadRunnable->UpdatePaths(ADBPath, AvdHomePath, GetPropCommand, bGetExtensionsViaSurfaceFlinger);
	}

	virtual void ExportDeviceProfile(const FString& OutPath, const FString& DeviceName) override
	{
		// instantiate an FPIEPreviewDeviceSpecifications instance and its values
		FPIEPreviewDeviceSpecifications DeviceSpecs;

		bool bOpenGL3x = false;
		{
			FScopeLock ExportLock(GetDeviceMapLock());

			const FAndroidDeviceInfo* DeviceInfo = GetDeviceMap().Find(DeviceName);
			if (DeviceInfo == nullptr)
			{
				FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, LOCTEXT("loc_ExportError_Message", "Device disconnected!"), LOCTEXT("loc_ExportError_Title", "File export error."));
				return;
			}

			// generic values
			DeviceSpecs.DevicePlatform = EPIEPreviewDeviceType::Android;
			DeviceSpecs.ResolutionX = DeviceInfo->ResolutionX;
			DeviceSpecs.ResolutionY = DeviceInfo->ResolutionY;
			DeviceSpecs.InsetsLeft = DeviceInfo->InsetsLeft;
			DeviceSpecs.InsetsTop = DeviceInfo->InsetsTop;
			DeviceSpecs.InsetsRight = DeviceInfo->InsetsRight;
			DeviceSpecs.InsetsBottom = DeviceInfo->InsetsBottom;
			DeviceSpecs.ResolutionYImmersiveMode = 0;
			DeviceSpecs.PPI = DeviceInfo->DeviceDPI;
			DeviceSpecs.ScaleFactors = { 0.25f, 0.5f, 0.75f, 1.0f };

			// Android specific values
			DeviceSpecs.AndroidProperties.Version = ANDROID_DEVICE_PROPERTIES_VERSION;
			DeviceSpecs.AndroidProperties.AndroidVersion = DeviceInfo->HumanAndroidVersion;
			DeviceSpecs.AndroidProperties.DeviceModel = DeviceInfo->Model;
			DeviceSpecs.AndroidProperties.DeviceMake = DeviceInfo->DeviceBrand;
			DeviceSpecs.AndroidProperties.GLVersion = DeviceInfo->OpenGLVersionString;
			DeviceSpecs.AndroidProperties.GPUFamily = DeviceInfo->GPUFamilyString;
			DeviceSpecs.AndroidProperties.VulkanVersion = DeviceInfo->VulkanVersion;
			DeviceSpecs.AndroidProperties.Hardware = DeviceInfo->Hardware;
			DeviceSpecs.AndroidProperties.DeviceBuildNumber = DeviceInfo->BuildNumber;
			// this is used in the same way as PlatformMemoryBucket..
			// to establish the nearest GB Android has a different rounding algo (hence 384 used here). See GenericPlatformMemory::GetMemorySizeBucket.
			DeviceSpecs.AndroidProperties.TotalPhysicalGB = FString::Printf(TEXT("%" UINT64_FMT),(((uint64)DeviceInfo->TotalPhysicalKB + 384 * 1024 - 1) / 1024 / 1024));
			
			DeviceSpecs.AndroidProperties.UsingHoudini = false;
			DeviceSpecs.AndroidProperties.VulkanAvailable = !(DeviceInfo->VulkanVersion.IsEmpty() || DeviceInfo->VulkanVersion.Contains(TEXT("0.0.0")));

			// OpenGL ES 3.x
			bOpenGL3x = DeviceInfo->OpenGLVersionString.Contains(TEXT("OpenGL ES 3"));
			// OpenGL ES 2.0 devices are no longer supported.
			if (!bOpenGL3x)
			{
				UE_LOGF(LogCore, Warning, "Cannot export device info, a minimum of OpenGL ES 3 is required.");
				return;
			}
		} // FScopeLock ExportLock released

		// create a JSon object from the above structure
		TSharedPtr<FJsonObject> JsonObject = FJsonObjectConverter::UStructToJsonObject<FPIEPreviewDeviceSpecifications>(DeviceSpecs);

		// remove IOS and switch fields
		JsonObject->RemoveField(TEXT("IOSProperties"));
		JsonObject->RemoveField(TEXT("switchProperties"));

		// serialize the JSon object to string
		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

		// export file to disk
		FFileHelper::SaveStringToFile(OutputString, *OutPath);
	} // end of virtual void ExportDeviceProfile(...)

private:

	// path to the adb command (local)
	FString ADBPath;

	FString SDKDirEnvVar;
	FString SDKRelativeExePath;
	FString AvdHomePath;
	FString GetPropCommand;
	bool bGetExtensionsViaSurfaceFlinger;

	FRunnableThread* DetectionThread;
	FAndroidDeviceDetectionRunnable* DetectionThreadRunnable;

	TMap<FString,FAndroidDeviceInfo> DeviceMap;
	FCriticalSection DeviceMapLock;
	FCriticalSection ADBPathCheckLock;


#if WITH_EDITOR
	FDelegateHandle DelegateHandle;

	// function will enumerate available Android devices that can export their profile to a json file
	// called (below) from AddAndroidConfigExportMenu()
	void AddAndroidConfigExportSubMenus(FMenuBuilder& InMenuBuilder)
	{
		TMap<FString, FAndroidDeviceInfo> AndroidDeviceMap;

		// lock device map and copy its contents
		{
			FCriticalSection* DeviceLock = GetDeviceMapLock();
			FScopeLock Lock(DeviceLock);
			AndroidDeviceMap = GetDeviceMap();
		}

		for (auto& Pair : AndroidDeviceMap)
		{
			FAndroidDeviceInfo& DeviceInfo = Pair.Value;

			FString ModelName = DeviceInfo.Model + TEXT("[") + DeviceInfo.DeviceBrand + TEXT("]");

			// lambda function called to open the save dialog and trigger device export
			auto LambdaSaveConfigFile = [DeviceName = Pair.Key, DefaultFileName = ModelName, this]()
			{
				TArray<FString> OutputFileName;
				FString DefaultFolder = FPaths::EngineContentDir() + TEXT("Editor/PIEPreviewDeviceSpecs/Android/");

				bool bResult = FDesktopPlatformModule::Get()->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
					LOCTEXT("PackagePluginDialogTitle", "Save platform configuration...").ToString(),
					DefaultFolder,
					DefaultFileName,
					TEXT("Json config file (*.json)|*.json"),
					0,
					OutputFileName);

				if (bResult && OutputFileName.Num())
				{
					ExportDeviceProfile(OutputFileName[0], DeviceName);
				}
			};

			InMenuBuilder.AddMenuEntry(
				FText::FromString(ModelName),
				FText(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"),
				FUIAction(FExecuteAction::CreateLambda(LambdaSaveConfigFile))
			);
		}
	}

	// function adds a sub-menu that will enumerate Android devices whose profiles can be exported json files
	void AddAndroidConfigExportMenu(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddSubMenu(
			LOCTEXT("loc_AddAndroidConfigExportMenu", "Export device settings"),
			LOCTEXT("loc_tip_AddAndroidConfigExportMenu", "Export device settings to a Json file."),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& Builder) { AddAndroidConfigExportSubMenus(Builder); }),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.SaveAll")
		);
	}

	// Android devices can export their profile to a json file which then can be used for PIE device simulations
	void CreatePIEPreviewMenu(FMenuBuilder& MenuBuilder)
	{
		// check to see if we have any connected devices
		bool bHasAndroidDevices = false;
		{
			FCriticalSection* DeviceLock = GetDeviceMapLock();
			FScopeLock Lock(DeviceLock);
			bHasAndroidDevices = GetDeviceMap().Num() > 0;
		}

		// add the config. export menu
		if (bHasAndroidDevices)
		{
			AddAndroidConfigExportMenu(MenuBuilder);
		}
	}
#endif
};


/**
 * Holds the target platform singleton.
 */
static TMap<FString, FAndroidDeviceDetection*> AndroidDeviceDetectionSingletons;


/**
 * Module for detecting android devices.
 */
class FAndroidDeviceDetectionModule : public IAndroidDeviceDetectionModule
{
public:
	/**
	 * Destructor.
	 */
	~FAndroidDeviceDetectionModule( )
	{
		for (auto It : AndroidDeviceDetectionSingletons)
		{
			delete It.Value;
		}
		AndroidDeviceDetectionSingletons.Empty();
	}

	virtual IAndroidDeviceDetection* GetAndroidDeviceDetection(const TCHAR* OverridePlatformName) override
	{
		FString Key(OverridePlatformName);
		FAndroidDeviceDetection* Value = AndroidDeviceDetectionSingletons.FindRef(Key);
		if (Value == nullptr)
		{
			Value = AndroidDeviceDetectionSingletons.Add(Key, new FAndroidDeviceDetection());
		}
		return Value;
	}
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FAndroidDeviceDetectionModule, AndroidDeviceDetection);
