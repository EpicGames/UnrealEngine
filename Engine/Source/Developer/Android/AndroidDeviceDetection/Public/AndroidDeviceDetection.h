// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidDeviceDetection.h: AndroidDeviceDetection module public header file.
=============================================================================*/

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/* Dependencies
 *****************************************************************************/

#include "Core.h"
#include "Modules/ModuleManager.h"

/* Interfaces
 *****************************************************************************/

#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
