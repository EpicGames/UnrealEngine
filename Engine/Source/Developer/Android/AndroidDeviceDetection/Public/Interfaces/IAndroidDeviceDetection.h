// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IAndroidDeviceDetection.h: Declares the IAndroidDeviceDetection interface.
=============================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"

template<typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs> class TMap;

struct FAndroidDeviceInfo
{
	FString DeviceId;
	
	FString AvdName;
	
	// Device serial number, used to route ADB commands to a specific device
	FString SerialNumber;

	// Device model name
	FString Model;

	// Device name
	FString DeviceName;

	// User-visible version of android installed (ro.build.version.release)
	FString HumanAndroidVersion;

	// Android SDK version supported by the device (ro.build.version.sdk - note: deprecated in 4 according to docs, but version 4 devices return an empty string when querying the 'replacement' SDK_INT)
	int32 SDKVersion = INDEX_NONE;

	FString Architecture;

	// List of supported OpenGL extensions (retrieved via SurfaceFlinger)
	FString GLESExtensions;

	// Supported GLES version (ro.opengles.version)
	int32 GLESVersion = INDEX_NONE;

	// Is the device authorized for USB communication?  If not, then none of the other properties besides the serial number will be valid
	bool bAuthorizedDevice = true;

	// TCP port number on our local host forwarded over adb to the device
	uint16 HostMessageBusPort = 0;

	// Holds pixel per inch value.
	int32 DeviceDPI = 0;

	// Holds the safe zones for the device
	float InsetsLeft;
	float InsetsTop;
	float InsetsRight;
	float InsetsBottom;

	// Holds the display resolution for the device
	int32 ResolutionX = 0;
	int32 ResolutionY = 0;

	// Holds the reported OpenGLES version.
	FString OpenGLVersionString;

	// device's vulkan version, in major.minor.patch format.
	FString VulkanVersion;

	// Holds the GPU family name.
	FString GPUFamilyString;

	// Holds the name of the manufacturer
	FString DeviceBrand;

	// device Hardware field. (or hardware.chipset for qcom)
	FString Hardware;

	// device's total mem (as reported from /proc/meminfo)
	uint32 TotalPhysicalKB = 0;

	// device BuildNumber field
	FString BuildNumber;
};


/**
 * Interface for AndroidDeviceDetection module.
 */
class IAndroidDeviceDetection
{
public:
	virtual void Initialize(const TCHAR* SDKDirectoryEnvVar, const TCHAR* SDKRelativeExePath, const TCHAR* GetPropCommand, bool bGetExtensionsViaSurfaceFlinger) = 0;
	virtual const TMap<FString,FAndroidDeviceInfo>& GetDeviceMap() = 0;
	virtual FCriticalSection* GetDeviceMapLock() = 0;
	virtual void UpdateADBPath() = 0;
	virtual FString GetADBPath() = 0;

	virtual void ExportDeviceProfile(const FString& OutPath, const FString& DeviceName) = 0;
protected:

	/**
	 * Virtual destructor
	 */
	virtual ~IAndroidDeviceDetection() = default;
};
