// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IAndroidDeviceDetectionModule.h: Declares the IAndroidDeviceDetectionModule interface.
=============================================================================*/

#pragma once

#include "Modules/ModuleInterface.h"

class IAndroidDeviceDetection;

/**
 * Interface for AndroidDeviceDetection module.
 */
class IAndroidDeviceDetectionModule
	: public IModuleInterface
{
public:
	/**
	 * Returns the android device detection singleton.
	 * @param AlternamePlatformName If a platform needs a separate detection instance, pass in an identifier here to create a new one
	 */
	virtual IAndroidDeviceDetection* GetAndroidDeviceDetection(const TCHAR* AlternamePlatformName=TEXT("")) = 0;

protected:

	/**
	 * Virtual destructor
	 */
	virtual ~IAndroidDeviceDetectionModule( ) = default;
};
