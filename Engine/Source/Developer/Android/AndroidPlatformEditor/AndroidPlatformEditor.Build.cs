// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AndroidPlatformEditor : ModuleRules
{
	public AndroidPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error; 
		BinariesSubFolder = "Android";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",				
                "EditorWidgets",
                "DesktopWidgets",
				"PropertyEditor",
				"SharedSettingsWidgets",
				"SourceControl",
				"AndroidRuntimeSettings",
                "AndroidDeviceDetection",
                "DesktopPlatform",
                "RenderCore",
                "RHI",
                "MaterialShaderQualitySettings",
				"MainFrame",
                "AudioSettingsEditor",
				"ShaderPlatformConfigEditor",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
			}
		);
	}
}
