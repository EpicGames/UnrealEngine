// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidLicenseDialog.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"
#include "Misc/FileHelper.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EngineVersion.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/AppStyle.h"
#include "Misc/SecureHash.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IMainFrameModule.h"

#include "AndroidRuntimeSettings.h"

#define LOCTEXT_NAMESPACE "AndroidLicenseDialog"

void SAndroidLicenseDialog::Construct(const FArguments& InArgs)
{
	bLicenseValid = false;

	// from Android SDK Tools 26.1.1
	FString LicenseFilename = FPaths::EngineDir() + TEXT("Source/ThirdParty/Android/package.xml");
	FString LicenseText = "Unable to read " + LicenseFilename;

	// Create file reader
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*LicenseFilename));
	if (FileReader)
	{
		// Create buffer for file input
		uint32 BufferSize = IntCastChecked<uint32>(FileReader->TotalSize());
		uint8* Buffer = (uint8*)FMemory::Malloc(BufferSize);
		FileReader->Serialize(Buffer, BufferSize);

		LicenseText = "Invalid license!";

		uint8 StartPattern[] = "<license id=\"android-sdk-license\" type=\"text\">";
		int32 StartPatternLength = strlen((char *)StartPattern);

		uint8* LicenseStart = Buffer;
		uint8* BufferEnd = Buffer + BufferSize - StartPatternLength;
		while (LicenseStart < BufferEnd)
		{
			if (!memcmp(LicenseStart, StartPattern, StartPatternLength))
			{
				break;
			}
			LicenseStart++;
		}

		if (LicenseStart < BufferEnd)
		{
			LicenseStart += StartPatternLength;

			uint8 EndPattern[] = "</license>";
			int32 EndPatternLength = strlen((char *)EndPattern);

			uint8* LicenseEnd = LicenseStart;
			BufferEnd = Buffer + BufferSize - EndPatternLength;
			while (LicenseEnd < BufferEnd)
			{
				if (!memcmp(LicenseEnd, EndPattern, EndPatternLength))
				{
					break;
				}
				LicenseEnd++;
			}

			if (LicenseEnd < BufferEnd)
			{
				int32 LicenseLength = IntCastChecked<int32>(LicenseEnd - LicenseStart);

				{
					const FUTF8ToTCHAR ConvertedString(reinterpret_cast<ANSICHAR*>(LicenseStart), LicenseLength);
					LicenseText = FString::ConstructFromPtrSize(ConvertedString.Get(), ConvertedString.Length());
				}

				FSHA1::HashBuffer(LicenseStart, LicenseLength, LicenseHash.Hash);

				bLicenseValid = true;
			}
		}
		FMemory::Free(Buffer);
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SAssignNew(ScrollBox, SScrollBox)
			.Style(FAppStyle::Get(), "ScrollBox")

			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SRichTextBlock)
					.Text(FText::FromString(LicenseText))
					.DecoratorStyleSet(&FAppStyle::Get())
					.AutoWrapText(true)
					.Justification(ETextJustify::Left)
				]
			]
		]

	+ SVerticalBox::Slot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(20, 5, 20, 5)
			.AutoWidth()
			[
				SNew(SButton)
				.IsEnabled(bLicenseValid)
				.OnClicked(this, &SAndroidLicenseDialog::OnAgree)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AndroidLicenseAgreement_Agree", "Agree"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(20, 5, 20, 5)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SAndroidLicenseDialog::OnCancel)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AndroidLicenseAgreement_Cancel", "Cancel"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	];
}

static FString GetLicensePath()
{
	auto &AndroidDeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection");
	IAndroidDeviceDetection* DeviceDetection = AndroidDeviceDetection.GetAndroidDeviceDetection();
	FString ADBPath = DeviceDetection->GetADBPath();

	if (!FPaths::FileExists(ADBPath))
	{
		return TEXT("");
	}

	// strip off the adb.exe part
	FString PlatformToolsPath;
	FString Filename;
	FString Extension;
	FPaths::Split(ADBPath, PlatformToolsPath, Filename, Extension);

	// remove the platform-tools part and point to licenses
	FPaths::NormalizeDirectoryName(PlatformToolsPath);
	FString LicensePath = PlatformToolsPath + "/../licenses";
	FPaths::CollapseRelativeDirectories(LicensePath);

	return LicensePath;
}

bool SAndroidLicenseDialog::HasLicense()
{
	FString LicensePath = GetLicensePath();

	if (LicensePath.IsEmpty())
	{
		return false;
	}

	// directory must exist
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*LicensePath))
	{
		return false;
	}

	// license file must exist
	FString LicenseFilename = LicensePath + "/android-sdk-license";
	if (!PlatformFile.FileExists(*LicenseFilename))
	{
		return false;
	}

	// contents must match hash of license text
	FString FileData = "";
	FFileHelper::LoadFileToString(FileData, *LicenseFilename);
	TArray<FString> lines;
	int32 lineCount = FileData.ParseIntoArray(lines, TEXT("\n"), true);

	FString LicenseString = LicenseHash.ToString().ToLower();
	for (FString &line : lines)
	{
		if (line.TrimStartAndEnd().Equals(LicenseString))
		{
			return true;
		}
	}

	// doesn't match
	return false;
}

void SAndroidLicenseDialog::SetLicenseAcceptedCallback(const FSimpleDelegate& InOnLicenseAccepted)
{
	OnLicenseAccepted = InOnLicenseAccepted;
}

FReply SAndroidLicenseDialog::OnAgree()
{
	FString LicensePath = GetLicensePath();

	if (!LicensePath.IsEmpty())
	{
		// create licenses directory if doesn't exist
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (!PlatformFile.DirectoryExists(*LicensePath))
		{
			PlatformFile.CreateDirectory(*LicensePath);
		}

		FString LicenseFilename = LicensePath + "/android-sdk-license";
		IFileHandle* FileHandle = PlatformFile.OpenWrite(*LicenseFilename);
		if (FileHandle)
		{
			FString HashText = TEXT("\015\012") + LicenseHash.ToString().ToLower();
			FileHandle->Write((const uint8*)TCHAR_TO_ANSI(*HashText), HashText.Len());
			delete FileHandle;
		}
		else
		{
			FText ErrorText = FText::Format(LOCTEXT("CouldntWriteLicense", "Couldn't write license file {0}. Make sure you have the permissions to modify the file and try again."), FText::FromString(LicenseFilename));
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorText.ToString(), TEXT("Error"));
		}
	}

	OnLicenseAccepted.ExecuteIfBound();

	TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()).ToSharedRef();
	FSlateApplication::Get().RequestDestroyWindow(ParentWindow);
	return FReply::Handled();
}

FReply SAndroidLicenseDialog::OnCancel()
{
	TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared()).ToSharedRef();
	FSlateApplication::Get().RequestDestroyWindow(ParentWindow);
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
