// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/SecureHash.h"

class SScrollBox;

/**
 * Credit screen widget that displays a scrolling list contributors.  
 */
class SAndroidLicenseDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAndroidLicenseDialog )
		{}
	SLATE_END_ARGS()

	/**
	 * Constructs the credits screen widgets                   
	 */
	void Construct(const FArguments& InArgs);

	bool HasLicense();

	void SetLicenseAcceptedCallback(const FSimpleDelegate& InOnLicenseAccepted);

private:
	bool bLicenseValid;

	FReply OnAgree();
	FReply OnCancel();

	FSHAHash LicenseHash;

	/** The widget that scrolls the license text */
	TSharedPtr<SScrollBox> ScrollBox;

	FSimpleDelegate OnLicenseAccepted;
};

