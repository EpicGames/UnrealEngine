// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "PropertyEditorModule.h"
#include "AndroidRuntimeSettings.h"
#include "AndroidTargetSettingsCustomization.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "AndroidSDKSettings.h"
#include "AndroidSDKSettingsCustomization.h"
#include "ISettingsModule.h"
#include "MaterialShaderQualitySettingsCustomization.h"
#include "MaterialShaderQualitySettings.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderingThread.h"
#include "ShaderPlatformQualitySettings.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "FAndroidPlatformEditorModule"


/**
 * Module for Android platform editor utilities
 */
class FAndroidPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register settings detail panel customization
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(
			UAndroidRuntimeSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FAndroidTargetSettingsCustomization::MakeInstance)
		);

		PropertyModule.RegisterCustomClassLayout(
			UAndroidSDKSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FAndroidSDKSettingsCustomization::MakeInstance)
			);

		FOnUpdateMaterialShaderQuality UpdateMaterials = FOnUpdateMaterialShaderQuality::CreateLambda([]()
		{
			FGlobalComponentRecreateRenderStateContext Recreate;
			FlushRenderingCommands();
			UMaterial::AllMaterialsCacheResourceShadersForRendering();
			UMaterialInstance::AllMaterialsCacheResourceShadersForRendering();
		});

		PropertyModule.RegisterCustomClassLayout(
			UShaderPlatformQualitySettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialShaderQualitySettingsCustomization::MakeInstance, UpdateMaterials)
			);

		PropertyModule.NotifyCustomizationModuleChanged();

		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Android",
				LOCTEXT("RuntimeSettingsName", "Android"),
				LOCTEXT("RuntimeSettingsDescription", "Project settings for Android apps"),
				GetMutableDefault<UAndroidRuntimeSettings>()
			);

 			SettingsModule->RegisterSettings("Project", "Platforms", "AndroidSDK",
 				LOCTEXT("SDKSettingsName", "Android SDK"),
 				LOCTEXT("SDKSettingsDescription", "Settings for Android SDK (for all projects)"),
 				GetMutableDefault<UAndroidSDKSettings>()
			);

			{
				static FName NAME_GLSL_ES3_1_ANDROID(TEXT("GLSL_ES3_1_ANDROID"));
				UShaderPlatformQualitySettings* AndroidMaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(NAME_GLSL_ES3_1_ANDROID);
				SettingsModule->RegisterSettings("Project", "Platforms", "AndroidOpenGLMobileQuality",
					LOCTEXT("AndroidOpenGLMobileQualitySettingsName", "Android Material Quality - OpenGL Mobile"),
					LOCTEXT("AndroidOpenGLMobileQualitySettingsDescription", "Settings for Android OpenGL Mobile material quality"),
					AndroidMaterialQualitySettings
				);
			}
			{
				static FName NAME_SF_VULKAN_ES31_ANDROID(TEXT("SF_VULKAN_ES31_ANDROID"));
				UShaderPlatformQualitySettings* AndroidMaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(NAME_SF_VULKAN_ES31_ANDROID);
				SettingsModule->RegisterSettings("Project", "Platforms", "AndroidVulkanQuality",
					LOCTEXT("AndroidVulkanQualitySettingsName", "Android Material Quality - Vulkan"),
					LOCTEXT("AndroidVulkanQualitySettingsDescription", "Settings for Android Vulkan material quality"),
					AndroidMaterialQualitySettings
				);
			}
			{
				static FName NAME_SF_VULKAN_SM5_ANDROID(TEXT("SF_VULKAN_SM5_ANDROID"));
				UShaderPlatformQualitySettings* AndroidMaterialQualitySettings = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(NAME_SF_VULKAN_SM5_ANDROID);
				SettingsModule->RegisterSettings("Project", "Platforms", "AndroidVulkanSM5Quality",
					LOCTEXT("AndroidVulkanSM5QualitySettingsName", "Android SM5 Material Quality - Vulkan"),
					LOCTEXT("AndroidVulkanSM5QualitySettingsDescription", "Settings for Android Vulkan SM5 material quality"),
					AndroidMaterialQualitySettings
				);
			}

		}


		auto AndroidRuntimeSettings = GetMutableDefault<UAndroidRuntimeSettings>();
		if (AndroidRuntimeSettings)
		{
			AndroidRuntimeSettings->OnPropertyChanged.AddLambda([this, AndroidRuntimeSettings](struct FPropertyChangedEvent& PropertyChangedEvent)
				{
					if (PropertyChangedEvent.Property != nullptr)
					{
						if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bBuildForES31) &&
							AndroidRuntimeSettings->bBuildForES31 == false)
						{
							FCoreDelegates::OnFeatureLevelDisabled.Broadcast(ERHIFeatureLevel::ES3_1, FName());
							return;
						}

						if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bSupportsVulkan) &&
							AndroidRuntimeSettings->bSupportsVulkan == false)
						{
							FCoreDelegates::OnFeatureLevelDisabled.Broadcast(ERHIFeatureLevel::ES3_1, LegacyShaderPlatformToShaderFormat(SP_VULKAN_ES3_1_ANDROID));
							return;
						}
					}

				});
		}

		// Force the SDK settings into a sane state initially so we can make use of them
		auto &TargetPlatformManagerModule = FModuleManager::LoadModuleChecked<ITargetPlatformManagerModule>("TargetPlatform");
		UAndroidSDKSettings * settings = GetMutableDefault<UAndroidSDKSettings>();
		settings->SetTargetModule(&TargetPlatformManagerModule);
		auto &AndroidDeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection");
		settings->SetDeviceDetection(AndroidDeviceDetection.GetAndroidDeviceDetection());
		settings->UpdateTargetModulePaths();
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Android");
			SettingsModule->UnregisterSettings("Project", "Platforms", "AndroidSDK");
			SettingsModule->UnregisterSettings("Project", "Platforms", "AndroidES31Quality");
			SettingsModule->UnregisterSettings("Project", "Platforms", "AndroidVulkanQuality");
		}
	}
};


IMPLEMENT_MODULE(FAndroidPlatformEditorModule, AndroidPlatformEditor);

#undef LOCTEXT_NAMESPACE
