// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidSDKSettings.h"
#include "Misc/Paths.h"
#include "Interfaces/IAndroidDeviceDetection.h"

//#include "EngineTypes.h"
#include "Interfaces/ITargetPlatformManagerModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidSDKSettings)

DEFINE_LOG_CATEGORY_STATIC(AndroidSDKSettings, Log, All);

UAndroidSDKSettings::UAndroidSDKSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

#if WITH_EDITOR
void UAndroidSDKSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateTargetModulePaths();	
}

void UAndroidSDKSettings::SetTargetModule(ITargetPlatformManagerModule * InTargetManagerModule)
{
	TargetManagerModule = InTargetManagerModule;
}

void UAndroidSDKSettings::SetDeviceDetection(IAndroidDeviceDetection * InAndroidDeviceDetection)
{
	AndroidDeviceDetection = InAndroidDeviceDetection;
}

void UAndroidSDKSettings::UpdateTargetModulePaths()
{
	TArray<FString> Keys;
	TArray<FString> Values;
	
	if (!SDKPath.Path.IsEmpty())
	{
		FPaths::NormalizeFilename(SDKPath.Path);
		Keys.Add(TEXT("ANDROID_HOME"));
		Values.Add(SDKPath.Path);
	}
	
	if (!NDKPath.Path.IsEmpty())
	{
		FPaths::NormalizeFilename(NDKPath.Path);
		Keys.Add(TEXT("NDKROOT"));
		Values.Add(NDKPath.Path);
	}
	
	if (!JavaPath.Path.IsEmpty())
	{
		FPaths::NormalizeFilename(JavaPath.Path);
		Keys.Add(TEXT("JAVA_HOME"));
		Values.Add(JavaPath.Path);
	}

	SaveConfig();
	
	if (Keys.Num() != 0)
	{
		TargetManagerModule->SetupEnvironmentVariables(Keys, Values);
		AndroidDeviceDetection->UpdateADBPath();
	}
}

#endif

