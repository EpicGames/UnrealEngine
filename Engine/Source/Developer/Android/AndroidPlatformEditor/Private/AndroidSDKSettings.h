// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "AndroidSDKSettings.generated.h"

#define UE_API ANDROIDPLATFORMEDITOR_API

class IAndroidDeviceDetection;
class ITargetPlatformManagerModule;

/**
 * Implements the settings for the Android SDK setup.
 */
UCLASS(MinimalAPI, config=Engine, globaluserconfig)
class UAndroidSDKSettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	// Location on disk of the Android SDK (falls back to ANDROID_HOME environment variable if this is left blank)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SDKConfig, Meta = (DisplayName = "Location of Android SDK (the directory usually contains 'android-sdk-')"))
	FDirectoryPath SDKPath;

	// Location on disk of the Android NDK (falls back to NDKROOT environment variable if this is left blank)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SDKConfig, Meta = (DisplayName = "Location of Android NDK (the directory usually contains 'android-ndk-')"))
	FDirectoryPath NDKPath;

	// Location on disk of Java (falls back to JAVA_HOME environment variable if this is left blank)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SDKConfig, Meta = (DisplayName = "Location of JAVA (the directory usually contains 'jdk')"))
	FDirectoryPath JavaPath;
	
	// Which SDK to package and compile Java with (a specific version, usually (without quotes): 'android-NN' or (without quotes) 'latest' for latest version on disk, or 'matchndk' to match the NDK API Level)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SDKConfig, Meta = (DisplayName = "SDK API Level (specific version e.g. 'android-NN', 'latest', or 'matchndk' - see tooltip)"))
	FString SDKAPILevel;

	// Which NDK to compile with (a specific version or (without quotes) 'latest' for latest version on disk). Note that choosing android-21 or later won't run on pre-5.0 devices.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = SDKConfig, Meta = (DisplayName = "NDK API Level (specific version or 'latest' - see tooltip)"))
	FString NDKAPILevel;


#if WITH_EDITOR
	// UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
	UE_API void SetTargetModule(ITargetPlatformManagerModule * TargetManagerModule);
	UE_API void SetDeviceDetection(IAndroidDeviceDetection * AndroidDeviceDetection);
	UE_API void UpdateTargetModulePaths();
	ITargetPlatformManagerModule * TargetManagerModule;
	IAndroidDeviceDetection * AndroidDeviceDetection;
#endif
};

#undef UE_API
