// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidSDKSettingsCustomization.h"
#include "Modules/ModuleManager.h"
#include "Layout/Visibility.h"
#include "UnrealClient.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "AndroidSDKSettings.h"
#include "PropertyHandle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"

#define LOCTEXT_NAMESPACE "AndroidSDKSettings"

//////////////////////////////////////////////////////////////////////////
// FAndroidSDKSettingsCustomization

TSharedRef<IDetailCustomization> FAndroidSDKSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAndroidSDKSettingsCustomization);
}

FAndroidSDKSettingsCustomization::FAndroidSDKSettingsCustomization()
{
	TargetPlatformManagerModule = &FModuleManager::LoadModuleChecked<ITargetPlatformManagerModule>("TargetPlatform");
}

void FAndroidSDKSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	SavedLayoutBuilder = &DetailLayout;
	BuildSDKPathSection(DetailLayout);
}


void FAndroidSDKSettingsCustomization::BuildSDKPathSection(IDetailLayoutBuilder& DetailLayout)
{
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
