// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidTargetSettingsCustomization.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SBoxPanel.h"
#include "Engine/GameEngine.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "AndroidRuntimeSettings.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

#include "SExternalImageReference.h"
#include "SHyperlinkLaunchURL.h"
#include "SPlatformSetupMessage.h"
#include "PlatformIconInfo.h"
#include "SourceControlHelpers.h"
#include "ManifestUpdateHelper.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/EngineBuildSettings.h"
#include "InstalledPlatformInfo.h"

#include "AndroidLicenseDialog.h"
#include "Interfaces/IMainFrameModule.h"
#include "Framework/Application/SlateApplication.h"
#include "ShaderPlatformConfigEditor.h"

#define LOCTEXT_NAMESPACE "AndroidRuntimeSettings"

//////////////////////////////////////////////////////////////////////////
// FAndroidTargetSettingsCustomization
namespace FAndroidTargetSettingsCustomizationConstants
{
	const FText DisabledTip = LOCTEXT("GitHubSourceRequiredToolTip", "This requires GitHub source.");
}

TSharedRef<IDetailCustomization> FAndroidTargetSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FAndroidTargetSettingsCustomization);
}

FAndroidTargetSettingsCustomization::FAndroidTargetSettingsCustomization()
	: LastLicenseChecktime(-1.0)
	, AndroidRelativePath(TEXT(""))
	, EngineAndroidPath(FPaths::EngineDir() + TEXT("Build/Android/Java"))
	, GameAndroidPath(FPaths::ProjectDir() + TEXT("Build/Android"))
	, EngineGooglePlayAppIDPath(EngineAndroidPath / TEXT("res") / TEXT("values") / TEXT("GooglePlayAppID.xml"))
	, GameGooglePlayAppIDPath(GameAndroidPath / TEXT("res") / TEXT("values") / TEXT("GooglePlayAppID.xml"))
	, EngineProguardPath(EngineAndroidPath / TEXT("proguard-project.txt"))
	, GameProguardPath(GameAndroidPath / TEXT("proguard-project.txt"))
	, EngineProjectPropertiesPath(EngineAndroidPath / TEXT("project.properties"))
	, GameProjectPropertiesPath(GameAndroidPath / TEXT("project.properties"))
{
	IconNames.Emplace(TEXT("res/drawable/icon.png"), LOCTEXT("SettingsIcon", "Icon"), FText::GetEmpty(), 48, 48, FPlatformIconInfo::Required);
	IconNames.Emplace(TEXT("res/drawable-ldpi/icon.png"), LOCTEXT("SettingsIcon_LDPI", "LDPI Icon"), FText::GetEmpty(), 36, 36, FPlatformIconInfo::Required);
	IconNames.Emplace(TEXT("res/drawable-mdpi/icon.png"), LOCTEXT("SettingsIcon_MDPI", "MDPI Icon"), FText::GetEmpty(), 48, 48, FPlatformIconInfo::Required);
	IconNames.Emplace(TEXT("res/drawable-hdpi/icon.png"), LOCTEXT("SettingsIcon_HDPI", "HDPI Icon"), FText::GetEmpty(), 72, 72, FPlatformIconInfo::Required);
	IconNames.Emplace(TEXT("res/drawable-xhdpi/icon.png"), LOCTEXT("SettingsIcon_XHDPI", "XHDPI Icon"), FText::GetEmpty(), 96, 96, FPlatformIconInfo::Required);

	LaunchImageNames.Emplace(TEXT("res/drawable/downloadimagev.png"), LOCTEXT("SettingsIcon_DownloadImageV", "Download Background Vertical Image"), FText::GetEmpty(), 720, 1280, FPlatformIconInfo::Required);
	LaunchImageNames.Emplace(TEXT("res/drawable/downloadimageh.png"), LOCTEXT("SettingsIcon_DownloadImageH", "Download Background Horizontal Image"), FText::GetEmpty(), 1280, 720, FPlatformIconInfo::Required);
	LaunchImageNames.Emplace(TEXT("res/drawable/splashscreen_portrait.png"), LOCTEXT("LaunchImage_Portrait", "Launch Portrait"), FText::GetEmpty(), 360, 640, FPlatformIconInfo::Required);
	LaunchImageNames.Emplace(TEXT("res/drawable/splashscreen_landscape.png"), LOCTEXT("LaunchImage_Landscape", "Launch Landscape"), FText::GetEmpty(), 640, 360, FPlatformIconInfo::Required);
}

void FAndroidTargetSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	SavedLayoutBuilder = &DetailLayout;

	BuildAppManifestSection(DetailLayout);
	BuildIconSection(DetailLayout);
	BuildLaunchImageSection(DetailLayout);
	AudioPluginWidgetManager.BuildAudioCategory(DetailLayout, FString(TEXT("Android")));

	UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigsFromSettingsModule(DetailLayout, "AndroidTargetPlatformSettings");
}

static void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* URL = Metadata.Find(TEXT("href"));
	
	if(URL)
	{
		FPlatformProcess::LaunchURL(**URL, nullptr, nullptr);
	}
}


void FAndroidTargetSettingsCustomization::BuildAppManifestSection(IDetailLayoutBuilder& DetailLayout)
{
	// Cache some categories
	IDetailCategoryBuilder& APKPackagingCategory = DetailLayout.EditCategory(TEXT("APK Packaging"));
	IDetailCategoryBuilder& BuildCategory = DetailLayout.EditCategory(TEXT("Build"));
	IDetailCategoryBuilder& AdvancedBuildCategory = DetailLayout.EditCategory(TEXT("AdvancedBuild"));
	AdvancedBuildCategory.InitiallyCollapsed(true);
	IDetailCategoryBuilder& SDKConfigCategory = DetailLayout.EditCategory(TEXT("Project SDK Override"));
	SDKConfigCategory.InitiallyCollapsed(true);
	IDetailCategoryBuilder& DebugSigningCategory = DetailLayout.EditCategory(TEXT("DebugSigning"));
	DebugSigningCategory.InitiallyCollapsed(true);

	IDetailCategoryBuilder& DistributionSigningCategory = DetailLayout.EditCategory(TEXT("DistributionSigning"));

	TSharedRef<SPlatformSetupMessage> PlatformSetupMessage = SNew(SPlatformSetupMessage, GameProjectPropertiesPath)
		.PlatformName(LOCTEXT("AndroidPlatformName", "Android"))
		.OnSetupClicked(this, &FAndroidTargetSettingsCustomization::CopySetupFilesIntoProject);

	SetupForPlatformAttribute = PlatformSetupMessage->GetReadyToGoAttribute();

	APKPackagingCategory.AddCustomRow(LOCTEXT("Warning", "Warning"), false)
		.WholeRowWidget
		[
			PlatformSetupMessage
		];

	APKPackagingCategory.AddCustomRow(LOCTEXT("UpgradeInfo", "Upgrade Info"), false)
		.WholeRowWidget
		[
			SNew(SBorder)
			.Padding(1)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(10, 10, 10, 10))
				.FillWidth(1.0f)
				[
					SNew(SRichTextBlock)
					.Text(LOCTEXT("UpgradeInfoMessage", "<RichTextBlock.TextHighlight>Note to users from 4.6 or earlier</>: We now <RichTextBlock.TextHighlight>GENERATE</> an AndroidManifest.xml when building, so if you have customized your .xml file, you will need to put all of your changes into the below settings. Note that we don't touch your AndroidManifest.xml that is in your project directory.\nAdditionally, we no longer use SigningConfig.xml, the settings are now set in the Distribution Signing section.\n\n<RichTextBlock.TextHighlight>NOTE</>: You must accept the SDK license agreement (click on button below) to use Gradle if it isn't grayed out."))
					.TextStyle(FAppStyle::Get(), "MessageLog")
					.DecoratorStyleSet(&FAppStyle::Get())
					.AutoWrapText(true)
					+ SRichTextBlock::HyperlinkDecorator(TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic(&OnBrowserLinkClicked))
				]
			]
		];

	APKPackagingCategory.AddCustomRow(LOCTEXT("AndroidSDKLicenses", "Android SDK Licenses"), false)
		.WholeRowWidget
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 5, 5, 5))
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked(this, &FAndroidTargetSettingsCustomization::OnAcceptSDKLicenseClicked)
				.IsEnabled(this, &FAndroidTargetSettingsCustomization::IsLicenseInvalid)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AcceptSDKLicense", "Accept SDK License"))
				]
			]
		];

	APKPackagingCategory.AddCustomRow(LOCTEXT("BuildFolderLabel", "Build Folder"), false)
		.IsEnabled(SetupForPlatformAttribute)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0, 1, 0, 1))
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BuildFolderLabel", "Build Folder"))
				.Font(DetailLayout.GetDetailFont())
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("OpenBuildFolderButton", "Open Build Folder"))
				.ToolTipText(LOCTEXT("OpenManifestFolderButton_Tooltip", "Opens the folder containing the build files in Explorer or Finder (it's recommended you check these in to revision control to share with your team)"))
				.OnClicked(this, &FAndroidTargetSettingsCustomization::OpenBuildFolder)
			]
		];

	SDKConfigCategory.AddCustomRow(LOCTEXT("SDKConfigInfo", "SDK Config Info"), false)
		.WholeRowWidget
		[
			SNew(SBorder)
			.Padding(1)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(10, 10, 10, 10))
				.FillWidth(1.0f)
				[
					SNew(SRichTextBlock)
					.Text(LOCTEXT("SDKConfigMessage", "Leave these fields blank to use global Android SDK project settings. Changing these settings will only affect this project."))
					.TextStyle(FAppStyle::Get(), "MessageLog")
					.DecoratorStyleSet(&FAppStyle::Get())
					.AutoWrapText(true)
				]
			]
		];

	TSharedRef<IPropertyHandle> SDKAPILevelOverrideProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, SDKAPILevelOverride));
	SDKConfigCategory.AddProperty(SDKAPILevelOverrideProperty);

	TSharedRef<IPropertyHandle> NDKAPILevelOverrideProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, NDKAPILevelOverride));
	SDKConfigCategory.AddProperty(NDKAPILevelOverrideProperty);

	// Signing category
	DistributionSigningCategory.AddCustomRow(LOCTEXT("SigningHyperlink", "Signing Hyperlink"), false)
		.WholeRowWidget
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SHyperlinkLaunchURL, TEXT("http://developer.android.com/tools/publishing/app-signing.html#releasemode"))
				.Text(LOCTEXT("AndroidDeveloperSigningPage", "Android Developer page on Signing for Distribution"))
				.ToolTipText(LOCTEXT("AndroidDeveloperSigningPageTooltip", "Opens a page that discusses the signing using keytool"))
			]
		];

	// Google Play category
	IDetailCategoryBuilder& GooglePlayCategory = DetailLayout.EditCategory(TEXT("GooglePlayServices"));
	
	TSharedRef<SPlatformSetupMessage> GooglePlaySetupMessage = SNew(SPlatformSetupMessage, GameGooglePlayAppIDPath)
		.PlatformName(LOCTEXT("GooglePlayPlatformName", "Google Play services"))
		.OnSetupClicked(this, &FAndroidTargetSettingsCustomization::CopyGooglePlayAppIDFileIntoProject);

	SetupForGooglePlayAttribute = GooglePlaySetupMessage->GetReadyToGoAttribute();

	GooglePlayCategory.AddCustomRow(LOCTEXT("Warning", "Warning"), false)
		.WholeRowWidget
		[
			GooglePlaySetupMessage
		];

	GooglePlayCategory.AddCustomRow(LOCTEXT("AppIDHyperlink", "App ID Hyperlink"), false)
		.WholeRowWidget
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SHyperlinkLaunchURL, TEXT("http://developer.android.com/google/index.html"))
				.Text(LOCTEXT("GooglePlayDeveloperPage", "Android Developer Page on Google Play services"))
				.ToolTipText(LOCTEXT("GooglePlayDeveloperPageTooltip", "Opens a page that discusses Google Play services"))
			]
		];

	TSharedRef<IPropertyHandle> EnabledProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bEnableGooglePlaySupport));
	GooglePlayCategory.AddProperty(EnabledProperty)
		.EditCondition(SetupForGooglePlayAttribute, NULL);

	TSharedRef<IPropertyHandle> AppIDProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, GamesAppID));
	AppIDProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FAndroidTargetSettingsCustomization::OnAppIDModified));
	GooglePlayCategory.AddProperty(AppIDProperty)
		.EditCondition(SetupForGooglePlayAttribute, NULL);

	TSharedRef<IPropertyHandle> SupportAdMobProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, bSupportAdMob));
	GooglePlayCategory.AddProperty(SupportAdMobProperty)
		.EditCondition(SetupForGooglePlayAttribute, NULL);

	TSharedRef<IPropertyHandle> AdMobAppIDProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, AdMobAppID));
	GooglePlayCategory.AddProperty(AdMobAppIDProperty);

	TSharedRef<IPropertyHandle> TagForChildDirectedTreatmentProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, TagForChildDirectedTreatment));
	GooglePlayCategory.AddProperty(TagForChildDirectedTreatmentProperty);

	TSharedRef<IPropertyHandle> TagForUnderAgeOfConsentProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, TagForUnderAgeOfConsent));
	GooglePlayCategory.AddProperty(TagForUnderAgeOfConsentProperty);

	TSharedRef<IPropertyHandle> MaxAdContentRatingProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, MaxAdContentRating));
	GooglePlayCategory.AddProperty(MaxAdContentRatingProperty);

	TSharedRef<IPropertyHandle> AdMobAdUnitIDProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, AdMobAdUnitID));
	AdMobAdUnitIDProperty->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> AdMobAdUnitIDsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, AdMobAdUnitIDs));
	GooglePlayCategory.AddProperty(AdMobAdUnitIDsProperty)
		.EditCondition(SetupForGooglePlayAttribute, NULL);

	TSharedRef<IPropertyHandle> GooglePlayLicenseKeyProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, GooglePlayLicenseKey));
	GooglePlayCategory.AddProperty(GooglePlayLicenseKeyProperty)
		.EditCondition(SetupForGooglePlayAttribute, NULL);

#define SETUP_ANDROIDARCH_PROP(ArchFragment, PropName, Category, Tip) \
	{ \
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, PropName)); \
		Category.AddProperty(PropertyHandle) \
			.IsEnabled(FInstalledPlatformInfo::Get().IsValidPlatformArchitecture(TEXT("Android"), ArchFragment)) \
			.ToolTip(FInstalledPlatformInfo::Get().IsValidPlatformArchitecture(TEXT("Android"), ArchFragment) ? Tip : FAndroidTargetSettingsCustomizationConstants::DisabledTip); \
	}

#define SETUP_SOURCEONLY_PROP(PropName, Category, Tip) \
	{ \
		TSharedRef<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UAndroidRuntimeSettings, PropName)); \
		Category.AddProperty(PropertyHandle) \
			.IsEnabled(FEngineBuildSettings::IsSourceDistribution()) \
			.ToolTip(FEngineBuildSettings::IsSourceDistribution() ? Tip : FAndroidTargetSettingsCustomizationConstants::DisabledTip); \
	}
	SETUP_ANDROIDARCH_PROP(TEXT("arm64"), bBuildForArm64, BuildCategory, LOCTEXT("BuildForArm64ToolTip", "Enable Arm64 CPU architecture support? (use at least NDK r11c, requires Lollipop (android-21) minimum)"));
	SETUP_ANDROIDARCH_PROP(TEXT("x64"), bBuildForX8664, BuildCategory, LOCTEXT("BuildForX8664ToolTip", "Enable X86-64 CPU architecture support?"));

	// @todo android fat binary: Put back in when we expose those
//	SETUP_SOURCEONLY_PROP(bSplitIntoSeparateApks, BuildCategory, LOCTEXT("SplitIntoSeparateAPKsToolTip", "If checked, CPU architectures and rendering types will be split into separate .apk files"));
}

bool FAndroidTargetSettingsCustomization::IsLicenseInvalid() const
{
	static bool bInvalid = true;

	// only check every 30 seconds after first time
	double CurrentTime = FApp::GetCurrentTime();
	if (LastLicenseChecktime < 0.0 || CurrentTime - LastLicenseChecktime >= 30.0)
	{
		const_cast<FAndroidTargetSettingsCustomization *>(this)->LastLicenseChecktime = CurrentTime;

		TSharedPtr<SAndroidLicenseDialog> LicenseDialog = SNew(SAndroidLicenseDialog);
		bInvalid = !LicenseDialog->HasLicense();
	}

	return bInvalid;
}

void FAndroidTargetSettingsCustomization::OnLicenseAccepted()
{
	LastLicenseChecktime = -1.0;
}

FReply FAndroidTargetSettingsCustomization::OnAcceptSDKLicenseClicked()
{
	// only show if don't have a valid license
	TSharedPtr<SAndroidLicenseDialog> LicenseDialog = SNew(SAndroidLicenseDialog);
	if (!LicenseDialog->HasLicense())
	{
		FSimpleDelegate LicenseAcceptedCallback = FSimpleDelegate::CreateSP(this, &FAndroidTargetSettingsCustomization::OnLicenseAccepted);
		LicenseDialog->SetLicenseAcceptedCallback(LicenseAcceptedCallback);

		const FText AndroidLicenseWindowTitle = LOCTEXT("AndroidLicenseUnrealEditor", "Android SDK License");

		TSharedPtr<SWindow> AndroidLicenseWindow =
			SNew(SWindow)
			.Title(AndroidLicenseWindowTitle)
			.ClientSize(FVector2D(600.f, 700.f))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::FixedSize)
			[
				LicenseDialog.ToSharedRef()
			];

		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		TSharedPtr<SWindow> ParentWindow = MainFrame.GetParentWindow();

		if (ParentWindow.IsValid())
		{
			FSlateApplication::Get().AddModalWindow(AndroidLicenseWindow.ToSharedRef(), ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(AndroidLicenseWindow.ToSharedRef());
		}
	}

	LastLicenseChecktime = -1.0;

	return FReply::Handled();
}

void FAndroidTargetSettingsCustomization::BuildIconSection(IDetailLayoutBuilder& DetailLayout)
{
	// Icon category
	IDetailCategoryBuilder& IconCategory = DetailLayout.EditCategory(TEXT("Icons"));

	IconCategory.AddCustomRow(LOCTEXT("IconsHyperlink", "Icons Hyperlink"), false)
		.WholeRowWidget
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SHyperlinkLaunchURL, TEXT("https://material.io/design/iconography/product-icons.html"))
				.Text(LOCTEXT("AndroidDeveloperIconographyPage", "Android Developer Page on Iconography"))
				.ToolTipText(LOCTEXT("AndroidDeveloperIconographyPageTooltip", "Opens a page on Android Iconography"))
			]
		];

	for (const FPlatformIconInfo& Info : IconNames)
	{
		const FString AutomaticImagePath = EngineAndroidPath / Info.IconPath;
		const FString TargetImagePath = GameAndroidPath / Info.IconPath;

		IconCategory.AddCustomRow(Info.IconName)
		.NameContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding( FMargin( 0, 1, 0, 1 ) )
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(Info.IconName)
				.Font(DetailLayout.GetDetailFont())
			]
		]
		.ValueContent()
		.MaxDesiredWidth(400.0f)
		.MinDesiredWidth(100.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SExternalImageReference, AutomaticImagePath, TargetImagePath)
				.FileDescription(Info.IconDescription)
				.RequiredSize(Info.IconRequiredSize)
				.MaxDisplaySize(FVector2D(FMath::Min(96, Info.IconRequiredSize.X), FMath::Min(96, Info.IconRequiredSize.Y)))
			]
		];
	}
}

void FAndroidTargetSettingsCustomization::BuildLaunchImageSection(IDetailLayoutBuilder& DetailLayout)
{
	// Add the launch images
	IDetailCategoryBuilder& LaunchImageCategory = DetailLayout.EditCategory(TEXT("LaunchImages"));
	LaunchImageCategory.AddCustomRow(LOCTEXT("LaunchImageInfo", "Launch Image Info"), false)
		.WholeRowWidget
		[
			SNew(SBorder)
			.Padding(1)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(10, 10, 10, 10))
				.FillWidth(1.0f)
				[
					SNew(SRichTextBlock)
					.Text(LOCTEXT("LaunchImageInfoMessage", "The <RichTextBlock.TextHighlight>Download Background</> image is used as the background for OBB downloading.  The <RichTextBlock.TextHighlight>Launch Portrait</> image is used as a splash screen for applications with Portrait, Reverse Portrait, Sensor Portrait, Sensor, or Full Sensor orientation.  The <RichTextBlock.TextHighlight>Launch Landscape</> image is used as a splash screen for applications with Landscape, Sensor Landscape, Reverse Landscape, Sensor, or Full Sensor orientation.\n\nThe launch images will be scaled to fit the device in the active orientation. Additional optional launch images may be provided as overrides for LDPI, MDPI, HDPI, and XHDPI by placing them in the project's corresponding Build/Android/res/drawable-* directory."))
					.TextStyle(FAppStyle::Get(), "MessageLog")
					.DecoratorStyleSet(&FAppStyle::Get())
					.AutoWrapText(true)
					+ SRichTextBlock::HyperlinkDecorator(TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic(&OnBrowserLinkClicked))
				]
			]
		];

	const FVector2D LaunchImageMaxSize(150.0f, 150.0f);

	for (const FPlatformIconInfo& Info : LaunchImageNames)
	{
		const FString AutomaticImagePath = EngineAndroidPath / Info.IconPath;
		const FString TargetImagePath = GameAndroidPath / Info.IconPath;

		LaunchImageCategory.AddCustomRow(Info.IconName)
			.NameContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0, 1, 0, 1))
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text(Info.IconName)
					.Font(DetailLayout.GetDetailFont())
				]
			]
			.ValueContent()
			.MaxDesiredWidth(400.0f)
			.MinDesiredWidth(100.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SExternalImageReference, AutomaticImagePath, TargetImagePath)
					.FileDescription(Info.IconDescription)
//					.RequiredSize(Info.IconRequiredSize)
					.MaxDisplaySize(LaunchImageMaxSize)
				]
			];
	}
}

FReply FAndroidTargetSettingsCustomization::OpenBuildFolder()
{
	const FString BuildFolder = FPaths::ConvertRelativePathToFull(FPaths::GetPath(GameProjectPropertiesPath));
	FPlatformProcess::ExploreFolder(*BuildFolder);

	return FReply::Handled();
}

void FAndroidTargetSettingsCustomization::CopySetupFilesIntoProject()
{
	// First copy the manifest, it must get copied
	FText ErrorMessage;
	if (!SourceControlHelpers::CopyFileUnderSourceControl(GameProjectPropertiesPath, EngineProjectPropertiesPath, LOCTEXT("ProjectProperties", "Project Properties"), /*out*/ ErrorMessage))
	{
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	else
	{
		// Now try to copy all of the icons, etc... (these can be ignored if the file already exists)
		for (const FPlatformIconInfo& Info : IconNames)
		{
			const FString EngineImagePath = EngineAndroidPath / Info.IconPath;
			const FString ProjectImagePath = GameAndroidPath / Info.IconPath;

			if (!FPaths::FileExists(ProjectImagePath))
			{
				SourceControlHelpers::CopyFileUnderSourceControl(ProjectImagePath, EngineImagePath, Info.IconName, /*out*/ ErrorMessage);
			}
		}

		// Now try to copy all of the launch images... (these can be ignored if the file already exists)
		for (const FPlatformIconInfo& Info : LaunchImageNames)
		{
			const FString EngineImagePath = EngineAndroidPath / Info.IconPath;
			const FString ProjectImagePath = GameAndroidPath / Info.IconPath;

			if (!FPaths::FileExists(ProjectImagePath))
			{
				SourceControlHelpers::CopyFileUnderSourceControl(ProjectImagePath, EngineImagePath, Info.IconName, /*out*/ ErrorMessage);
			}
		}

		// and copy the other files (aren't required)
		//SourceControlHelpers::CopyFileUnderSourceControl(GameProguardPath, EngineProguardPath, LOCTEXT("Proguard", "Proguard Settings"), /*out*/ ErrorMessage);
	}

	SavedLayoutBuilder->ForceRefreshDetails();
}

void FAndroidTargetSettingsCustomization::CopyGooglePlayAppIDFileIntoProject()
{
	FText ErrorMessage;
	if (!SourceControlHelpers::CopyFileUnderSourceControl(GameGooglePlayAppIDPath, EngineGooglePlayAppIDPath, LOCTEXT("GooglePlayAppID", "GooglePlayAppID.xml"), /*out*/ ErrorMessage))
	{
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	SavedLayoutBuilder->ForceRefreshDetails();
}

void FAndroidTargetSettingsCustomization::OnAppIDModified()
{
	const FString NewIDString = GetDefault<UAndroidRuntimeSettings>()->GamesAppID;

	if (NewIDString.Len() > 0 && !FCString::IsNumeric(*NewIDString))
	{
		FNotificationInfo Info(LOCTEXT("InvalidGamesAppID", "The Games App ID you provided is invalid"));
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}

	if (FPaths::FileExists(GameGooglePlayAppIDPath))
	{
		FManifestUpdateHelper Updater(GameGooglePlayAppIDPath);

		const FString AppIDTag(TEXT("name=\"app_id\">"));
		const FString ClosingTag(TEXT("</string>"));
		Updater.ReplaceKey(AppIDTag, ClosingTag, NewIDString);

		Updater.Finalize(GameGooglePlayAppIDPath);
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
