// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "TargetPlatformAudioCustomization.h"

class IDetailLayoutBuilder;

//////////////////////////////////////////////////////////////////////////
// FAndroidTargetSettingsCustomization

class FAndroidTargetSettingsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	FAndroidTargetSettingsCustomization();

	void BuildAppManifestSection(IDetailLayoutBuilder& DetailLayout);
	void BuildIconSection(IDetailLayoutBuilder& DetailLayout);
	void BuildLaunchImageSection(IDetailLayoutBuilder& DetailLayout);

	// Navigates to the build files in the explorer or finder
	FReply OpenBuildFolder();

	// Copies the setup files for the platform into the project
	void CopySetupFilesIntoProject();

	// Copies the strings.xml file for the platform into the project
	void CopyGooglePlayAppIDFileIntoProject();

	// Called when the app id is modified
	void OnAppIDModified();

	// Called when License accepted
	void OnLicenseAccepted();

	// returns whether Android SDK license valid
	bool IsLicenseInvalid() const;

	// Show license agreement for user to accept
	FReply OnAcceptSDKLicenseClicked();

private:
	double LastLicenseChecktime;

	const FString AndroidRelativePath;

	const FString EngineAndroidPath;
	const FString GameAndroidPath;

	const FString EngineGooglePlayAppIDPath;
	const FString GameGooglePlayAppIDPath;

	const FString EngineProguardPath;
	const FString GameProguardPath;

	const FString EngineProjectPropertiesPath;
	const FString GameProjectPropertiesPath;

	TArray<struct FPlatformIconInfo> IconNames;
	TArray<struct FPlatformIconInfo> LaunchImageNames;

	// Is the manifest writable?
	TAttribute<bool> SetupForPlatformAttribute;

	// Is the App ID string writable?
	TAttribute<bool> SetupForGooglePlayAttribute;

	FAudioPluginWidgetManager AudioPluginWidgetManager;
 	IDetailLayoutBuilder* SavedLayoutBuilder;
};
