// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AndroidTargetPlatform : ModuleRules
{
	public AndroidTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		BinariesSubFolder = "Android";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.Android);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TargetPlatform",
                "DesktopPlatform",
				"AndroidDeviceDetection",
                "AudioPlatformConfiguration"
            }
		);

        if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
            PrivateIncludePathModuleNames.Add("TextureCompressor");     //@todo android: AndroidTargetPlatform.Build
        }

        PublicDefinitions.Add("WITH_OGGVORBIS=1");
	}
}
