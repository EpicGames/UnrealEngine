// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Android/AndroidPlatformProperties.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Common/TargetPlatformBase.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "IAndroidTargetPlatformModule.h"

/**
 * Module for the Android target platform.
 */
class FAndroidTargetPlatformModule : public ITargetPlatformModule
{
public:
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms)
	{

	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls) override
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}
};

IMPLEMENT_MODULE( FAndroidTargetPlatformModule, AndroidTargetPlatform);
