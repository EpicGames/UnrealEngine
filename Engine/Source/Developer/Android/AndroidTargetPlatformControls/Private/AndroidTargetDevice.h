// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidTargetDevice.h: Declares the AndroidTargetDevice class.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatform.h"
#include "Templates/SharedPointer.h"
#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

class FAndroidTargetDevice;
class FTargetDeviceId;
class ITargetPlatform;
struct FTargetDeviceProcessInfo;
enum class ETargetDeviceFeatures;
enum class ETargetDeviceTypes;

/**
 * Type definition for shared pointers to instances of FAndroidTargetDevice.
 */
typedef TSharedPtr<class FAndroidTargetDevice, ESPMode::ThreadSafe> FAndroidTargetDevicePtr;

/**
 * Type definition for shared references to instances of FAndroidTargetDevice.
 */
typedef TSharedRef<class FAndroidTargetDevice, ESPMode::ThreadSafe> FAndroidTargetDeviceRef;

/**
 * Type definition for shared references to instances of FAndroidTargetDevice.
 */
typedef TSharedPtr<class FAndroidTargetDeviceOutput, ESPMode::ThreadSafe> FAndroidTargetDeviceOutputPtr;

/**
 * Implements a Android target device.
 */
class FAndroidTargetDevice : public ITargetDevice
{
public:

	/**
	 * Creates and initializes a new Android target device.
	 *
	 * @param InTargetPlatformControls - The target platform controls.
	 * @param InSerialNumber - The ADB serial number of the target device.
	 * @param InAndroidVariant - The variant of the Android platform, i.e. ETC2, DXT or ASTC.
	 */
	FAndroidTargetDevice(const ITargetPlatformControls& InTargetPlatformControls, const FString& InDeviceId, const FString& InAndroidVariant)
		: DeviceId{InDeviceId}
		, AndroidVariant(InAndroidVariant)
		, bIsDeviceAuthorized(false)
		, AndroidSDKVersion(INDEX_NONE)
		, TargetPlatformControls(InTargetPlatformControls)
	{ }

public:
	const FString& GetDeviceId() const
	{
		return DeviceId;
	}
	
	/**
	 * Sets the device's connection state.
	 *
	 * @param bInConnected - Whether the device is connected.
	 */
	void SetConnected(bool bInConnected)
	{
		bConnected = bInConnected;
	}

	/**
	 * Sets the device's authorization state.
	 *
	 * @param bInConnected - Whether the device is authorized for USB communications.
	 */
	void SetAuthorized(bool bInIsAuthorized)
	{
		bIsDeviceAuthorized = bInIsAuthorized;
	}

	/**
	 * Sets the device's OS/SDK versions.
	 *
	 * @param InSDKVersion - Android SDK version of the device.
	 * @param InReleaseVersion - Android Release (human-readable) version of the device.
	 */
	void SetVersions(int32 InSDKVersion, const FString& InReleaseVersion)
	{
		AndroidSDKVersion = InSDKVersion;
		AndroidVersionString = InReleaseVersion;
	}

	/**
	 * Sets the architecture.
	 *
	 * @param InArchitecture - The architecture to set.
	 */
	void SetArchitecture(FString InArchitecture)
	{
		Architecture = MoveTemp(InArchitecture);
	}

	/**
	 * Sets the name.
	 *
	 * @param InName - The name to set.
	 */
	void SetName(FString InName)
	{
		Name = MoveTemp(InName);
	}

	/**
	 * Sets the device name.
	 *
	 * @param InDeviceName - The device name to set.
	 */
	void SetDeviceName(const FString& InDeviceName)
	{
		DeviceName = InDeviceName;
	}

	/**
	 * Sets the device name.
	 *
	 * @param InModel - The device name to set.
	 */
	void SetModel(const FString& InModel)
	{
		Model = InModel;
	}

	/**
	 * Sets the serial number
	 *
	 * @param InSerialNumber - The serial number to set.
	 */
	void SetSerialNumber(FString InSerialNumber)
	{
		SerialNumber = MoveTemp(InSerialNumber);
	}

	const FString& GetSerialNumber() const
	{
		return SerialNumber;
	}

public:

	//~ Begin ITargetDevice Interface
	virtual bool Connect() override
	{
		return true;
	}

	virtual void Disconnect() override
	{
	}

	virtual ETargetDeviceTypes GetDeviceType() const override
	{
		//@TODO: How to distinguish between a Tablet and a Phone (or a TV microconsole, etc...), and is it important?
		return ETargetDeviceTypes::Tablet;
	}

	virtual FTargetDeviceId GetId() const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), DeviceId);
	}

	virtual FString GetName() const override
	{
		// we need a unique name for all devices, so use human usable model name and the unique id
		return Name;
	}

	virtual FString GetModelId() const override
	{
		return Model;
	}

	virtual FString GetOSVersion() const override
	{
		return AndroidVersionString;
	}

	virtual FStringView GetArchitecture() const override
	{
		return Architecture;
	}

	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override
	{
		return *(TargetPlatformControls.GetTargetPlatformSettings());
	}
	virtual const class ITargetPlatformControls& GetPlatformControls() const override
	{
		return TargetPlatformControls;
	}

	virtual FString GetOperatingSystemName() override;

	virtual int32 GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) override;

	virtual bool IsConnected() override
	{
		return bConnected;
	}

	virtual bool IsDefault() const override
	{
		return true;
	}

	virtual bool IsAuthorized() const override
	{
		return bIsDeviceAuthorized;
	}

	virtual bool PowerOn() override;
	virtual bool PowerOff(bool Force) override;

	// Return true if the devices can be grouped in an aggregate (All_<platform>_devices_on_<host>) proxy
	virtual bool IsPlatformAggregated() const override
	{
		return true;
	}

	// the name of the aggregate (All_<platform>_devices_on_<host>) proxy
	virtual FString GetAllDevicesName() const override;

	// the default variant (texture compression) of the aggregate (All_<platform>_devices_on_<host>) proxy
	virtual FName GetAllDevicesDefaultVariant() const override
	{
		// The Android platform has an aggregate (All_<platform>_devices_on_<host>) entry in the Project Launcher
		// Multi is the default texture format
		return "Android_Multi";
	}

	virtual bool Reboot(bool bReconnect = false) override;
	virtual bool TerminateLaunchedProcess(const FString& ProcessIdentifier) override;
	virtual bool SupportsFeature(ETargetDeviceFeatures Feature) const override;
	virtual bool TerminateProcess(const int64 ProcessId) override;
	virtual void SetUserCredentials(const FString& UserName, const FString& UserPassword) override;
	virtual bool GetUserCredentials(FString& OutUserName, FString& OutUserPassword) override;
	virtual void ExecuteConsoleCommand(const FString& ExecCommand) const override;
	virtual ITargetDeviceOutputPtr CreateDeviceOutputRouter(FOutputDevice* Output) const override;
	//~ End ITargetDevice Interface

	/** Full filename for ADB executable. */
	static bool GetAdbFullFilename(FString& OutFilename);

protected:

	/**
	 * Executes an SDK command with the specified command line on this device only using ADB.
	 *
	 * @param Params - The command line parameters.
	 * @param OutStdOut - Optional pointer to a string that will hold the command's output log.
	 * @param OutStdErr - Optional pointer to a string that will hold the error message, if any.
	 *
	 * @return true on success, false otherwise.
	 */
	bool ExecuteAdbCommand( const FString& Params, FString* OutStdOut, FString* OutStdErr ) const;

protected:

	FString DeviceId;
	
	// The variant of the Android platform, i.e. ETC2, DXT or ASTC.
	FString AndroidVariant;

	// Holds a flag indicating whether the device is currently connected.
	bool bConnected;

	// Holds a flag indicating whether the device is USB comms authorized (if not, most other values aren't valid but we still want to show the device as detected but unready)
	bool bIsDeviceAuthorized;

	// Holds the Android SDK version
	int32 AndroidSDKVersion;

	// Holds the Android Release version string (e.g., "2.3" or "4.2.2")
	FString AndroidVersionString;

	FString Architecture;
	
	// Holds the name.
	FString Name;
	
	// Holds the device name.
	FString DeviceName;

	// Holds the device model.
	FString Model;

	// Holds the serial number (from ADB devices) of this target device.
	FString SerialNumber;

	// Holds a reference to the device's target platform.
	const ITargetPlatformControls& TargetPlatformControls;
};


#include "AndroidTargetDevice.inl" // IWYU pragma: export
