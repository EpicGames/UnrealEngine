// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

/* ITargetDevice interface
 *****************************************************************************/
#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "HAL/PlatformProcess.h"
#include "AndroidTargetDeviceOutput.h"

class FAndroidTargetDevice;
struct FTargetDeviceProcessInfo;
enum class ETargetDeviceFeatures;

inline FString FAndroidTargetDevice::GetOperatingSystemName()
{
	if (!AndroidVersionString.IsEmpty())
	{
		return FString::Printf(TEXT("Android %s, API level %d"), *AndroidVersionString, AndroidSDKVersion);
	}
	else
	{
		return TEXT("Android");
	}
}

inline int32 FAndroidTargetDevice::GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) 
{
	return 0;
}


inline bool FAndroidTargetDevice::Reboot( bool bReconnect )
{
	if (!ExecuteAdbCommand(TEXT("reboot"), NULL, NULL))
	{
		return false;
	}

	return true;
}

inline bool FAndroidTargetDevice::PowerOn()
{
	check(DeviceId.StartsWith(TEXT("avd-")));

	if (FString AndroidHomeDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("ANDROID_HOME")); !AndroidHomeDirectory.IsEmpty())
	{
		return bool{UE::HAL::FProcess{{.Uri = *FPaths::Combine(AndroidHomeDirectory, TEXT("emulator"), TEXT("emulator")), .Arguments = *(TEXT("-avd ") + DeviceId.RightChop(4)), .bDetached = true, .bHidden = true}}};
	}

	return false;
}

inline bool FAndroidTargetDevice::PowerOff(bool Force)
{
	return ExecuteAdbCommand(TEXT("reboot -p"), nullptr, nullptr);
}

inline FString FAndroidTargetDevice::GetAllDevicesName() const
{
	return FString::Printf(TEXT("All_%s_On_%s"), *(GetPlatformSettings().IniPlatformName()), FPlatformProcess::ComputerName());
}

// cancel the running application
inline bool FAndroidTargetDevice::TerminateLaunchedProcess(const FString& ProcessIdentifier)
{
	FString AdbCommand = FString::Printf(TEXT("shell am force-stop '%s' -s %s"), *ProcessIdentifier, *SerialNumber);
	return ExecuteAdbCommand(AdbCommand, nullptr, nullptr);
}

inline bool FAndroidTargetDevice::SupportsFeature(ETargetDeviceFeatures Feature) const
{
	switch (Feature)
	{
	case ETargetDeviceFeatures::PowerOff:
		return bIsDeviceAuthorized;

	case ETargetDeviceFeatures::PowerOn:
		return SerialNumber.IsEmpty() && DeviceId.StartsWith(TEXT("avd-"));

	case ETargetDeviceFeatures::Reboot:
		return bIsDeviceAuthorized;

	default:
		return false;
	}
}


inline bool FAndroidTargetDevice::TerminateProcess( const int64 ProcessId )
{
	return false;
}


inline void FAndroidTargetDevice::SetUserCredentials( const FString& UserName, const FString& UserPassword )
{
}


inline bool FAndroidTargetDevice::GetUserCredentials( FString& OutUserName, FString& OutUserPassword )
{
	return false;
}

inline void FAndroidTargetDevice::ExecuteConsoleCommand(const FString& ExecCommand) const
{
	FString AdbCommand = FString::Printf(TEXT("shell \"am broadcast -a android.intent.action.RUN -e cmd '%s'\""), *ExecCommand);
	ExecuteAdbCommand(AdbCommand, nullptr, nullptr);
}

inline ITargetDeviceOutputPtr FAndroidTargetDevice::CreateDeviceOutputRouter(FOutputDevice* Output) const
{
	FAndroidTargetDeviceOutputPtr DeviceOutputPtr = MakeShareable(new FAndroidTargetDeviceOutput());
	if (DeviceOutputPtr->Init(*this, Output))
	{
		return DeviceOutputPtr;
	}

	return nullptr;
}

/* FAndroidTargetDevice implementation
 *****************************************************************************/

inline bool FAndroidTargetDevice::GetAdbFullFilename(FString& OutFilename)
{
	TOptional<FString> ResultPath;

	// get the SDK binaries folder
	FString AndroidDirectory = FPlatformMisc::GetEnvironmentVariable(TEXT("ANDROID_HOME"));
	if (AndroidDirectory.Len() == 0)
	{
		return false;
	}

#if PLATFORM_WINDOWS
	OutFilename = FString::Printf(TEXT("%s\\platform-tools\\adb.exe"), *AndroidDirectory);
#else
	OutFilename = FString::Printf(TEXT("%s/platform-tools/adb"), *AndroidDirectory);
#endif

	return true;
}

inline bool FAndroidTargetDevice::ExecuteAdbCommand( const FString& CommandLine, FString* OutStdOut, FString* OutStdErr ) const
{
	FString Filename;
	if (!GetAdbFullFilename(Filename))
	{
		return false;
	}
	
	// execute the command
	int32 ReturnCode;
	FString DefaultError;

	// make sure there's a place for error output to go if the caller specified NULL
	if (!OutStdErr)
	{
		OutStdErr = &DefaultError;
	}

	FString CommandLineWithDevice;
	// the devices command should never include a specific device
	if (CommandLine == TEXT("devices"))
	{
		CommandLineWithDevice = CommandLine;
	}
	else
	{
		CommandLineWithDevice = FString::Printf(TEXT("-s %s %s"), *SerialNumber, *CommandLine);
	}

	FPlatformProcess::ExecProcess(*Filename, *CommandLineWithDevice, &ReturnCode, OutStdOut, OutStdErr);

	if (ReturnCode != 0)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("The Android SDK command '%s' failed to run. Return code: %d, Error: %s\n"), *CommandLineWithDevice, ReturnCode, **OutStdErr);

		return false;
	}

	return true;
}
