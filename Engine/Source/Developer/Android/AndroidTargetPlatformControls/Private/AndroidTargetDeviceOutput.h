// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Containers/UnrealString.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/ITargetDeviceOutput.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/ThreadSafeCounter.h"

class FAndroidTargetDevice;

class FAndroidDeviceOutputReaderRunnable : public FRunnable
{
public:
	FAndroidDeviceOutputReaderRunnable(const FString& AdbFilename, const FString& DeviceSerialNumber, FOutputDevice* Output);
	
	// FRunnable interface.
	virtual bool Init(void) override;
	virtual void Exit(void) override; 
	virtual void Stop(void) override;
	virtual uint32 Run(void) override;

private:
	bool StartLogcatProcess(void);

private:
	// > 0 if we've been asked to abort work in progress at the next opportunity
	FThreadSafeCounter	StopTaskCounter;
	
	FString				AdbFilename;
	FString				DeviceSerialNumber;
	FOutputDevice*		Output;
	
	void*				LogcatReadPipe;
	void*				LogcatWritePipe;
	FProcHandle			LogcatProcHandle;
};

/**
 * Implements a Android target device.
 */
class FAndroidTargetDeviceOutput : public ITargetDeviceOutput
{
public:
	bool Init(const FAndroidTargetDevice& TargetDevice, FOutputDevice* Output);
	
private:
	TUniquePtr<FRunnableThread>						DeviceOutputThread;
	FString											DeviceSerialNumber;
	FString											DeviceName;
};

#include "AndroidTargetDeviceOutput.inl" // IWYU pragma: export
