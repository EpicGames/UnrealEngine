// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

#include "CoreTypes.h"
#include "CoreFwd.h"

class FAndroidDeviceOutputReaderRunnable;
class FAndroidTargetDevice;
class FAndroidTargetDeviceOutput;

inline FAndroidDeviceOutputReaderRunnable::FAndroidDeviceOutputReaderRunnable(const FString& InAdbFilename, const FString& InDeviceSerialNumber, FOutputDevice* InOutput) 
	: StopTaskCounter(0)
	, AdbFilename(InAdbFilename)
	, DeviceSerialNumber(InDeviceSerialNumber)
	, Output(InOutput)
	, LogcatReadPipe(nullptr)
	, LogcatWritePipe(nullptr)
{
}

inline bool FAndroidDeviceOutputReaderRunnable::StartLogcatProcess(void)
{
	FString Params = FString::Printf(TEXT(" -s %s logcat UE:V DEBUG:V *:S -v tag"), *DeviceSerialNumber);
	LogcatProcHandle = FPlatformProcess::CreateProc(*AdbFilename, *Params, true, false, false, NULL, 0, NULL, LogcatWritePipe);
	return LogcatProcHandle.IsValid();
}

inline bool FAndroidDeviceOutputReaderRunnable::Init(void) 
{ 
	FPlatformProcess::CreatePipe(LogcatReadPipe, LogcatWritePipe);
	return StartLogcatProcess();
}

inline void FAndroidDeviceOutputReaderRunnable::Exit(void) 
{
	if (LogcatProcHandle.IsValid())
	{
		FPlatformProcess::CloseProc(LogcatProcHandle);
	}
	FPlatformProcess::ClosePipe(LogcatReadPipe, LogcatWritePipe);
}

inline void FAndroidDeviceOutputReaderRunnable::Stop(void)
{
	StopTaskCounter.Increment();
}

inline uint32 FAndroidDeviceOutputReaderRunnable::Run(void)
{
	FString LogcatOutput;
	
	while (StopTaskCounter.GetValue() == 0 && LogcatProcHandle.IsValid())
	{
		if (!FPlatformProcess::IsProcRunning(LogcatProcHandle))
		{
			// When user plugs out USB cable adb process stops
			// Keep trying to restore adb connection until code that uses this object will not kill us
			Output->Serialize(TEXT("Trying to restore connection to device..."), ELogVerbosity::Log, NAME_None);
			FPlatformProcess::CloseProc(LogcatProcHandle);
			if (StartLogcatProcess())
			{
				FPlatformProcess::Sleep(1.0f);
			}
			else
			{
				Output->Serialize(TEXT("Failed to start adb proccess"), ELogVerbosity::Log, NAME_None);
				Stop();
			}
		}
		else
		{
			LogcatOutput.Append(FPlatformProcess::ReadPipe(LogcatReadPipe));

			if (LogcatOutput.Len() > 0)
			{
				TArray<FString> OutputLines;
				LogcatOutput.ParseIntoArray(OutputLines, TEXT("\n"), false);

				if (!LogcatOutput.EndsWith(TEXT("\n")))
				{
					// partial line at the end, do not serialize it until we receive remainder
					LogcatOutput = OutputLines.Last();
					OutputLines.RemoveAt(OutputLines.Num() - 1);
				}
				else
				{
					LogcatOutput.Reset();
				}

				for (int32 i = 0; i < OutputLines.Num(); ++i)
				{
					Output->Serialize(*OutputLines[i], ELogVerbosity::Log, NAME_None);
				}
			}
			
			FPlatformProcess::Sleep(0.1f);
		}
	}

	return 0;
}

inline bool FAndroidTargetDeviceOutput::Init(const FAndroidTargetDevice& TargetDevice, FOutputDevice* Output)
{
	check(Output);
	// Output will be produced by background thread
	check(Output->CanBeUsedOnAnyThread());
	
	DeviceSerialNumber = TargetDevice.GetSerialNumber();
	DeviceName = TargetDevice.GetName();
		
	FString AdbFilename;
	if (FAndroidTargetDevice::GetAdbFullFilename(AdbFilename))
	{
		auto* Runnable = new FAndroidDeviceOutputReaderRunnable(AdbFilename, DeviceSerialNumber, Output);
		DeviceOutputThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(Runnable, TEXT("FAndroidDeviceOutputReaderRunnable")));
		return true;
	}
		
	return false;
}
	
