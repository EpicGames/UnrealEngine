// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidTargetPlatform.inl: Implements the FAndroidTargetPlatformControls class.
=============================================================================*/


/* FAndroidTargetPlatformControls structors
 *****************************************************************************/

#include "AndroidTargetPlatformControls.h"
#include "AndroidTargetPlatformSettings.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"
#include "Serialization/Archive.h"
#include "Misc/FileHelper.h"
#include "Misc/SecureHash.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Modules/ModuleManager.h"
#include "Misc/SecureHash.h"
#include "AnalyticsEventAttribute.h"


#if WITH_ENGINE
#include "AudioCompressionSettings.h"
#include "Sound/SoundWave.h"
#include "TextureResource.h"
#endif

#define LOCTEXT_NAMESPACE "FAndroidTargetPlatformControls"

class Error;
class FAndroidTargetDevice;
class FConfigCacheIni;
class FModuleManager;
class FScopeLock;
class FStaticMeshLODSettings;
class FTargetDeviceId;
class FTSTicker;
class IAndroidDeviceDetectionModule;
class UTexture;
class UTextureLODSettings;
struct FAndroidDeviceInfo;
enum class ETargetPlatformFeatures;

static FString GetLicensePath()
{
	auto &AndroidDeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection");
	IAndroidDeviceDetection* DeviceDetection = AndroidDeviceDetection.GetAndroidDeviceDetection();
	FString ADBPath = DeviceDetection->GetADBPath();

	if (!FPaths::FileExists(ADBPath))
	{
		return TEXT("");
	}

	// strip off the adb.exe part
	FString PlatformToolsPath;
	FString Filename;
	FString Extension;
	FPaths::Split(ADBPath, PlatformToolsPath, Filename, Extension);

	// remove the platform-tools part and point to licenses
	FPaths::NormalizeDirectoryName(PlatformToolsPath);
	FString LicensePath = PlatformToolsPath + "/../licenses";
	FPaths::CollapseRelativeDirectories(LicensePath);

	return LicensePath;
}

#if WITH_ENGINE
static bool GetLicenseHash(FSHAHash& LicenseHash)
{
	bool bLicenseValid = false;

	// from Android SDK Tools 25.2.3
	FString LicenseFilename = FPaths::EngineDir() + TEXT("Source/ThirdParty/Android/package.xml");

	// Create file reader
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*LicenseFilename));
	if (FileReader)
	{
		// Create buffer for file input
		uint32 BufferSize = IntCastChecked<uint32>(FileReader->TotalSize());
		uint8* Buffer = (uint8*)FMemory::Malloc(BufferSize);
		FileReader->Serialize(Buffer, BufferSize);

		uint8 StartPattern[] = "<license id=\"android-sdk-license\" type=\"text\">";
		int32 StartPatternLength = strlen((char *)StartPattern);

		uint8* LicenseStart = Buffer;
		uint8* BufferEnd = Buffer + BufferSize - StartPatternLength;
		while (LicenseStart < BufferEnd)
		{
			if (!memcmp(LicenseStart, StartPattern, StartPatternLength))
			{
				break;
			}
			LicenseStart++;
		}

		if (LicenseStart < BufferEnd)
		{
			LicenseStart += StartPatternLength;

			uint8 EndPattern[] = "</license>";
			int32 EndPatternLength = strlen((char *)EndPattern);

			uint8* LicenseEnd = LicenseStart;
			BufferEnd = Buffer + BufferSize - EndPatternLength;
			while (LicenseEnd < BufferEnd)
			{
				if (!memcmp(LicenseEnd, EndPattern, EndPatternLength))
				{
					break;
				}
				LicenseEnd++;
			}

			if (LicenseEnd < BufferEnd)
			{
				int32 LicenseLength = IntCastChecked<int32>(LicenseEnd - LicenseStart);
				FSHA1::HashBuffer(LicenseStart, LicenseLength, LicenseHash.Hash);
				bLicenseValid = true;
			}
		}
		FMemory::Free(Buffer);
	}

	return bLicenseValid;
}
#endif

static bool HasLicense()
{
#if WITH_ENGINE
	FString LicensePath = GetLicensePath();

	if (LicensePath.IsEmpty())
	{
		return false;
	}

	// directory must exist
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*LicensePath))
	{
		return false;
	}

	// license file must exist
	FString LicenseFilename = LicensePath + "/android-sdk-license";
	if (!PlatformFile.FileExists(*LicenseFilename))
	{
		return false;
	}

	FSHAHash LicenseHash;
	if (!GetLicenseHash(LicenseHash))
	{
		return false;
	}

	// contents must match hash of license text
	FString FileData = "";
	FFileHelper::LoadFileToString(FileData, *LicenseFilename);
	TArray<FString> lines;
	int32 lineCount = FileData.ParseIntoArray(lines, TEXT("\n"), true);

	FString LicenseString = LicenseHash.ToString().ToLower();
	for (FString &line : lines)
	{
		if (line.TrimStartAndEnd().Equals(LicenseString))
		{
			return true;
		}
	}
#endif

	// doesn't match
	return false;
}

FAndroidTargetPlatformControls::FAndroidTargetPlatformControls(bool bInIsClient, ITargetPlatformSettings* InTargetPlatformSettings, const TCHAR* FlavorName, const TCHAR* OverrideIniPlatformName)
	: TNonDesktopTargetPlatformControlsBase(bInIsClient, InTargetPlatformSettings, FlavorName, OverrideIniPlatformName)
	, DeviceDetection(nullptr)

{
	TickDelegate = FTickerDelegate::CreateRaw(this, &FAndroidTargetPlatformControls::HandleTicker);
	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 4.0f);

	AndroidTargetPlatformSettings = (FAndroidTargetPlatformSettings*)TargetPlatformSettings;
}


FAndroidTargetPlatformControls::~FAndroidTargetPlatformControls()
{
	 FTSTicker::RemoveTicker(TickDelegateHandle);
}


/* ITargetPlatform overrides
 *****************************************************************************/

void FAndroidTargetPlatformControls::GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const
{
	OutDevices.Reset();

	for (auto Iter = Devices.CreateConstIterator(); Iter; ++Iter)
	{
		OutDevices.Add(Iter.Value());
	}
}

ITargetDevicePtr FAndroidTargetPlatformControls::GetDefaultDevice( ) const
{
	// return the first device in the list
	if (Devices.Num() > 0)
	{
		auto Iter = Devices.CreateConstIterator();
		if (Iter)
		{
			return Iter.Value();
		}
	}

	return nullptr;
}

ITargetDevicePtr FAndroidTargetPlatformControls::GetDevice( const FTargetDeviceId& DeviceId )
{
	if (DeviceId.GetPlatformName() == PlatformName())
	{
		return Devices.FindRef(DeviceId.GetDeviceName());
	}

	return nullptr;
}


bool FAndroidTargetPlatformControls::IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const
{
	OutDocumentationPath = FString("Shared/Tutorials/SettingUpAndroidTutorial");
	return true;
}

int32 FAndroidTargetPlatformControls::CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const
{
	OutDocumentationPath = TEXT("Platforms/Android/GettingStarted");

	int32 bReadyToBuild = ETargetPlatformReadyStatus::Ready;
	if (!IsSdkInstalled(bProjectHasCode, OutTutorialPath))
	{
		bReadyToBuild |= ETargetPlatformReadyStatus::SDKNotFound;
	}

	// need to check license was accepted
	if (!HasLicense())
	{
		OutTutorialPath.Empty();
		CustomizedLogMessage = LOCTEXT("AndroidLicenseNotAcceptedMessageDetail", "SDK License must be accepted in the Android project settings to deploy your app to the device.");
		bReadyToBuild |= ETargetPlatformReadyStatus::LicenseNotAccepted;
	}

	return bReadyToBuild;
}

void FAndroidTargetPlatformControls::GetPlatformSpecificProjectAnalytics(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray) const
{
	TTargetPlatformControlsBase<FAndroidPlatformProperties>::GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);

	AppendAnalyticsEventAttributeArray(AnalyticsParamArray,
		TEXT("AndroidVariant"), GetAndroidVariantName(),
		TEXT("SupportsVulkan"), AndroidTargetPlatformSettings->SupportsVulkan(),
		TEXT("SupportsVulkanSM5"), AndroidTargetPlatformSettings->SupportsVulkanSM5(),
		TEXT("SupportsES31"), AndroidTargetPlatformSettings->SupportsES31()
	);

	AppendAnalyticsEventConfigBool(AnalyticsParamArray, TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bPackageForMetaQuest"), GEngineIni);
	AppendAnalyticsEventConfigBool(AnalyticsParamArray, TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bPackageForOpenXRImmersive"), GEngineIni);
}

#if WITH_ENGINE

FName FAndroidTargetPlatformControls::FinalizeVirtualTextureLayerFormat(FName Format) const
{
#if WITH_EDITOR
	// Remap non-ETC variants to ETC

	// VirtualTexture Format was already run through the ordinary texture remaps to change AutoDXT to ASTC or ETC
	// this then runs again
	// currently it forces all ASTC to ETC
	// this is needed because the runtime virtual texture encoder only supports ETC

	// code dupe with IOSTargetPlatform

	// @todo Oodle: restrict this so it's only done when needed for RVT, not for all VT that would be better left as ASTC ; UE-212640

	const static FName VTRemap[][2] =
	{
		{ { FName(TEXT("ASTC_RGB")) },			{ AndroidTexFormat::NameETC2_RGB } },
		{ { FName(TEXT("ASTC_RGBA")) },			{ AndroidTexFormat::NameETC2_RGBA } },
		{ { FName(TEXT("ASTC_RGBA_HQ")) },		{ AndroidTexFormat::NameETC2_RGBA } },
//		{ { FName(TEXT("ASTC_RGB_HDR")) },		{ NameRGBA16F } }, // ?
		{ { FName(TEXT("ASTC_RGBAuto")) },		{ AndroidTexFormat::NameAutoETC2 } },
		{ { FName(TEXT("ASTC_NormalAG")) },		{ AndroidTexFormat::NameETC2_RGB } },
		{ { AndroidTexFormat::NameASTC_NormalRG },	{ AndroidTexFormat::NameETC2_RG11 } },
		{ { AndroidTexFormat::NameASTC_NormalLA	},	{ AndroidTexFormat::NameETC2_RG11 } },
		{ { AndroidTexFormat::NameDXT1 },		{ AndroidTexFormat::NameETC2_RGB } },
		{ { AndroidTexFormat::NameDXT5 },		{ AndroidTexFormat::NameETC2_RGBA } },
		{ { AndroidTexFormat::NameAutoDXT },	{ AndroidTexFormat::NameAutoETC2 } }
	};

	for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(VTRemap); RemapIndex++)
	{
		if (VTRemap[RemapIndex][0] == Format)
		{
			return VTRemap[RemapIndex][1];
		}
	}
#endif
	return Format;
}
#endif //WITH_ENGINE

bool FAndroidTargetPlatformControls::SupportsVariants() const
{
	return true;
}


/* FAndroidTargetPlatformControls implementation
 *****************************************************************************/
void FAndroidTargetPlatformControls::InitializeDeviceDetection()
{
	DeviceDetection = FModuleManager::LoadModuleChecked<IAndroidDeviceDetectionModule>("AndroidDeviceDetection").GetAndroidDeviceDetection();
	DeviceDetection->Initialize(TEXT("ANDROID_HOME"),
#if PLATFORM_WINDOWS
		TEXT("platform-tools\\adb.exe"),
#else
		TEXT("platform-tools/adb"),
#endif
		TEXT("shell getprop"), true);
}

bool FAndroidTargetPlatformControls::ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const
{
	bool bIsMaliBugIndex = false;
	const uint16 MaliBugIndexMaxDiff = 16;
	uint16 LastIndex = Indices[0];
	for (int32 i = 1; i < NumIndices; ++i)
	{
		uint16 CurrentIndex = Indices[i];
		if ((FMath::Abs(LastIndex - CurrentIndex) > MaliBugIndexMaxDiff))
		{
			bIsMaliBugIndex = true;
			break;
		}
		else
		{
			LastIndex = CurrentIndex;
		}
	}
	return bIsMaliBugIndex;
}

/* FAndroidTargetPlatformControls callbacks
 *****************************************************************************/

bool FAndroidTargetPlatformControls::HandleTicker( float DeltaTime )
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAndroidTargetPlatform_HandleTicker);

	if (DeviceDetection == nullptr)
	{
		InitializeDeviceDetection();
		checkf(DeviceDetection != nullptr, TEXT("A target platform didn't create a device detection object in InitializeDeviceDetection()!"));
	}

	TArray<FStringView> ConnectedDeviceIds;

	{
		FScopeLock ScopeLock(DeviceDetection->GetDeviceMapLock());

		for (const auto& Pair : DeviceDetection->GetDeviceMap())
		{
			const FAndroidDeviceInfo& DeviceInfo = Pair.Value;

			// see if this device is already known
			if (FAndroidTargetDevicePtr TestDevice = Devices.FindRef(Pair.Key))
			{
				// ignore if authorization didn't change
				if (DeviceInfo.bAuthorizedDevice == TestDevice->IsAuthorized() && DeviceInfo.SerialNumber == TestDevice->GetSerialNumber())
				{
					ConnectedDeviceIds.Add(TestDevice->GetDeviceId());
					continue;
				}

				// remove it to add again
				TestDevice->SetConnected(false);
				Devices.Remove(Pair.Key);

				OnDeviceLost().Broadcast(TestDevice.ToSharedRef());
			}

			// check if this platform is supported by the extensions and version
			if (!SupportedByExtensionsString(DeviceInfo.GLESExtensions, DeviceInfo.GLESVersion))
			{
				continue;
			}

			// create target device
			FAndroidTargetDevicePtr& Device = Devices.Add(Pair.Key);

			Device = CreateTargetDevice(*this, DeviceInfo.DeviceId, GetAndroidVariantName());

			// we need a unique name for all devices, so use human usable display name and the unique id
			if (!DeviceInfo.SerialNumber.IsEmpty())
			{
				if (!DeviceInfo.Model.IsEmpty())
				{
					Device->SetName((!DeviceInfo.AvdName.IsEmpty() ? DeviceInfo.AvdName : DeviceInfo.Model) + TEXT(" (") + DeviceInfo.SerialNumber + TEXT(")"));
				}
				else
				{
					Device->SetName(DeviceInfo.SerialNumber + TEXT(" [UNAUTHORIZED]"));
				}
				
				Device->SetConnected(true);
			}
			else
			{
				Device->SetName(DeviceInfo.AvdName + TEXT(" [OFFLINE]"));
				Device->SetConnected(false);
			}

			Device->SetModel(DeviceInfo.Model);
			Device->SetDeviceName(DeviceInfo.DeviceName);
			Device->SetAuthorized(DeviceInfo.bAuthorizedDevice);
			Device->SetVersions(DeviceInfo.SDKVersion, DeviceInfo.HumanAndroidVersion);
			Device->SetArchitecture(DeviceInfo.Architecture);
			Device->SetSerialNumber(DeviceInfo.SerialNumber);

			ITargetPlatformControls::OnDeviceDiscovered().Broadcast(Device.ToSharedRef());
			
			ConnectedDeviceIds.Add(Device->GetDeviceId());
		}
	}

	// remove disconnected devices
	for (auto Iter = Devices.CreateIterator(); Iter; ++Iter)
	{
		if (!ConnectedDeviceIds.Contains(Iter.Key()))
		{
			FAndroidTargetDevicePtr RemovedDevice = Iter.Value();
			RemovedDevice->SetConnected(false);

			Iter.RemoveCurrent();

			OnDeviceLost().Broadcast(RemovedDevice.ToSharedRef());
		}
	}

	return true;
}

FAndroidTargetDeviceRef FAndroidTargetPlatformControls::CreateNewDevice(const FAndroidDeviceInfo &DeviceInfo)
{
	return MakeShareable(new FAndroidTargetDevice(*this, DeviceInfo.DeviceId, GetAndroidVariantName()));
}

FAndroidTargetDevicePtr FAndroidTargetPlatformControls::CreateTargetDevice(const ITargetPlatformControls& InTargetPlatform, const FString& InDeviceId, const FString& InAndroidVariant) const
{
	return MakeShareable(new FAndroidTargetDevice(InTargetPlatform, InDeviceId, InAndroidVariant));
}

#if WITH_ENGINE
void FAndroidTargetPlatformControls::GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	// FAndroidTargetPlatformControls aside from being the base class for all the concrete android target platforms
	//	it is also usable on its own as "flavorless" Android
	//	but I don't understand how that's supposed to work or what that's supposed to mean
	//	and no information has been forthcoming
	check(Texture);

	// Supported in ES3.2 with ASTC
	const bool bSupportCompressedVolumeTexture = AndroidTargetPlatformSettings->SupportsTextureFormatCategory(EAndroidTextureFormatCategory::ASTC);
	// FWIW bSupportCompressedVolumeTexture should be true for Android_DXT but this is setting it to false

	// OpenGL ES has F32 textures but doesn't allow linear filtering unless OES_texture_float_linear
	const bool bSupportFilteredFloat32Textures = false;

	// optionaly compress landscape weightmaps for a mobile rendering
	bool bCompressLandscapeWeightMaps = false;
	GetTargetPlatformSettings()->GetConfigSystem()->GetBool(TEXT("/Script/Engine.RendererSettings"), TEXT("r.Mobile.CompressLandscapeWeightMaps"), bCompressLandscapeWeightMaps, GEngineIni);

	TArray<FName>& LayerFormats = OutFormats.AddDefaulted_GetRef();
	int32 BlockSize = 1; // this looks wrong? should be 4 for FAndroid_DXTTargetPlatform ? - it is wrong, but BlockSize is ignored

	GetDefaultTextureFormatNamePerLayer(LayerFormats, this->GetTargetPlatformSettings(), this, Texture, bSupportCompressedVolumeTexture, BlockSize, bSupportFilteredFloat32Textures);

	for (FName& TextureFormatName : LayerFormats)
	{
		// @todo Oodle: this should not be here
		//	should be in GetDefaultTextureFormatNamePerLayer
		//	so that 4x4 checks can be applied correctly, etc.
		if (Texture->LODGroup == TEXTUREGROUP_Terrain_Weightmap && bCompressLandscapeWeightMaps)
		{
			TextureFormatName = AndroidTexFormat::NameAutoDXT;
		}

		if (Texture->GetTextureClass() == ETextureClass::Cube)
		{
			FTextureFormatSettings FormatSettings;
			Texture->GetDefaultFormatSettings(FormatSettings);
			// TC_EncodedReflectionCapture is no longer used and could be deleted
			if (FormatSettings.CompressionSettings == TC_EncodedReflectionCapture && !FormatSettings.CompressionNone)
			{
				TextureFormatName = FName(TEXT("ETC2_RGBA"));
			}
		}

		for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(AndroidTexFormat::GenericRemap); ++RemapIndex)
		{
			if (TextureFormatName == AndroidTexFormat::GenericRemap[RemapIndex][0])
			{
				TextureFormatName = AndroidTexFormat::GenericRemap[RemapIndex][1];
			}
		}
	}
}

void FAndroid_ASTCTargetPlatformControls::GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	FAndroidTargetPlatformControls::GetTextureFormats(Texture, OutFormats);

	// L+A mode for normal map compression
	const bool bSupportsNormalLA = GetTargetPlatformSettings()->SupportsFeature(ETargetPlatformFeatures::NormalmapLAEncodingMode);

	// perform any remapping away from defaults
	TArray<FName>& LayerFormats = OutFormats.Last();
	for (FName& TextureFormatName : LayerFormats)
	{
		if (bSupportsNormalLA && TextureFormatName == AndroidTexFormat::NameBC5)
		{
			TextureFormatName = AndroidTexFormat::NameASTC_NormalLA;
			continue;
		}

		for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(AndroidTexFormat::ASTCRemap); ++RemapIndex)
		{
			if (TextureFormatName == AndroidTexFormat::ASTCRemap[RemapIndex][0])
			{
				TextureFormatName = AndroidTexFormat::ASTCRemap[RemapIndex][1];
				break;
			}
		}

		if (Texture && Texture->GetTextureClass() == ETextureClass::Volume)
		{
			// Fallback to ETC for 3D textures, as ASTC for volume textures is unsupported on certain devices like the Pixel 10
			const static FName VolumeTexRemap[][2] =
			{
				{ AndroidTexFormat::NameAutoASTC,	AndroidTexFormat::NameAutoETC2	},
				{ FName(TEXT("ASTC_RGBA")),			AndroidTexFormat::NameETC2_RGBA	},
				{ FName(TEXT("ASTC_RGB")),			AndroidTexFormat::NameETC2_RGB	},
				{ FName(TEXT("ASTC_RGBA_HQ")),		AndroidTexFormat::NameETC2_RGBA	}
			};

			for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(VolumeTexRemap); ++RemapIndex)
			{
				if (TextureFormatName == VolumeTexRemap[RemapIndex][0])
				{
					TextureFormatName = VolumeTexRemap[RemapIndex][1];
					break;
				}
			}
		}
	}

	bool bSupportASTCHDR = AndroidTargetPlatformSettings->UsesASTCHDR();

	if (!bSupportASTCHDR)
	{
		for (FName& TextureFormatName : LayerFormats)
		{
			if (TextureFormatName == AndroidTexFormat::NameASTC_RGB_HDR)
			{
				TextureFormatName = GetTargetPlatformSettings()->GetFallbackASTCHDR();
			}
		}
	}
}

void FAndroid_DXTTargetPlatformControls::GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	FAndroidTargetPlatformControls::GetTextureFormats(Texture, OutFormats);

	bool bSupportsDX11Formats = false; // assume Android DXT does not support BC6/7

	if (!bSupportsDX11Formats)
	{
		TArray<FName>& LayerFormats = OutFormats.Last();

		for (FName& TextureFormatName : LayerFormats)
		{
			if (TextureFormatName == AndroidTexFormat::NameBC6H)
			{
				TextureFormatName = AndroidTexFormat::NameRGBA16F;
			}
			else if (TextureFormatName == AndroidTexFormat::NameBC7)
			{
				TextureFormatName = AndroidTexFormat::NameDXT5;
			}
		}
	}
}

void FAndroid_ETC2TargetPlatformControls::GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	FAndroidTargetPlatformControls::GetTextureFormats(Texture, OutFormats);

	// perform any remapping away from defaults
	TArray<FName>& LayerFormats = OutFormats.Last();
	for (FName& TextureFormatName : LayerFormats)
	{
		for (int32 RemapIndex = 0; RemapIndex < UE_ARRAY_COUNT(AndroidTexFormat::ETCRemap); ++RemapIndex)
		{
			if (TextureFormatName == AndroidTexFormat::ETCRemap[RemapIndex][0])
			{
				TextureFormatName = AndroidTexFormat::ETCRemap[RemapIndex][1];
				break;
			}
		}
	}
}

void FAndroid_MultiTargetPlatformControls::GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const
{
	// Ask each platform variant to choose texture formats
	for (ITargetPlatformControls* Platform : FormatTargetPlatforms)
	{
		TArray< TArray<FName> > PlatformFormats;
		Platform->GetTextureFormats(Texture, PlatformFormats);
		for (TArray<FName>& FormatPerLayer : PlatformFormats)
		{
			// For multiformat case we have to disable L+A normal map compression as only ASTC textures support it 
			for (FName& TextureFormatName : FormatPerLayer)
			{
				if (TextureFormatName == AndroidTexFormat::NameASTC_NormalLA)
				{
					TextureFormatName = AndroidTexFormat::NameASTC_NormalRG;
				}
			}

			OutFormats.AddUnique(FormatPerLayer);
		}
	}
}

#endif

FText FAndroid_MultiTargetPlatformControls::DisplayName() const
{
	return FText::Format(LOCTEXT("Android_Multi", "Android (Multi:{0})"), FText::FromString(FormatTargetString));
}

#undef LOCTEXT_NAMESPACE
