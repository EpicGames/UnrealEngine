// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Delegates/IDelegateInstance.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Containers/Ticker.h"
#include "Misc/ScopeLock.h"
#include "Android/AndroidPlatformProperties.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Common/TargetPlatformControlsBase.h"
#include "Interfaces/IAndroidDeviceDetection.h"
#include "Interfaces/IAndroidDeviceDetectionModule.h"
#include "AndroidTargetDevice.h"
#include "AndroidTargetPlatformSettings.h"

#if WITH_ENGINE
#include "Engine/Texture.h"
#include "Internationalization/Text.h"
#include "StaticMeshResources.h"
#endif // WITH_ENGINE


class FTargetDeviceId;
class IAndroidDeviceDetection;
class ITargetPlatform;
class UTextureLODSettings;
enum class ETargetPlatformFeatures;

/**
 * FAndroidTargetPlatformControls, abstraction for cooking Android platforms
 */
class FAndroidTargetPlatformControls
	: public TNonDesktopTargetPlatformControlsBase<FAndroidPlatformProperties>
{
public:

	/**
	 * Default constructor.
	 */
	ANDROIDTARGETPLATFORMCONTROLS_API FAndroidTargetPlatformControls(bool bInIsClient, ITargetPlatformSettings* TargetPlatformSettings, const TCHAR* FlavorName, const TCHAR* OverrideIniPlatformName = nullptr);

	/**
	 * Destructor
	 */
	ANDROIDTARGETPLATFORMCONTROLS_API virtual ~FAndroidTargetPlatformControls();

public:

	/**
	 * Gets the name of the Android platform variant, i.e. ASTC, ETC2, DXT, etc.
	 *
	 * @param Variant name.
	 */
	FString GetAndroidVariantName() const
	{
		return PlatformInfo->PlatformFlavor.ToString();
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual void EnableDeviceCheck(bool OnOff) override {}

	virtual bool AddDevice(const FString& DeviceName, bool bDefault) override
	{
		return false;
	}

	ANDROIDTARGETPLATFORMCONTROLS_API virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const override;

	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override
	{
		return true;
	}

	ANDROIDTARGETPLATFORMCONTROLS_API virtual ITargetDevicePtr GetDefaultDevice() const override;

	ANDROIDTARGETPLATFORMCONTROLS_API virtual ITargetDevicePtr GetDevice(const FTargetDeviceId& DeviceId) override;

	ANDROIDTARGETPLATFORMCONTROLS_API virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const override;

	ANDROIDTARGETPLATFORMCONTROLS_API virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override;

	ANDROIDTARGETPLATFORMCONTROLS_API virtual void GetPlatformSpecificProjectAnalytics(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray) const override;

	virtual bool SupportsCompressedNonPOT() const
	{
		// most formats do support non-POT compressed textures
		return true;
	}

#if WITH_ENGINE
	ANDROIDTARGETPLATFORMCONTROLS_API virtual void GetTextureFormats(const UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override;
	ANDROIDTARGETPLATFORMCONTROLS_API virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const override;
#endif //WITH_ENGINE

	ANDROIDTARGETPLATFORMCONTROLS_API virtual bool SupportsVariants() const override;

	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override
	{
		OutSection = TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings");
		InBoolKeys.Add(TEXT("bBuildForArm64"));	InBoolKeys.Add(TEXT("bBuildForX8664"));
		InBoolKeys.Add(TEXT("bBuildForES31")); InBoolKeys.Add(TEXT("bBuildWithHiddenSymbolVisibility"));
		InBoolKeys.Add(TEXT("bSaveSymbols")); InStringKeys.Add(TEXT("NDKAPILevel"));
	}

	ANDROIDTARGETPLATFORMCONTROLS_API virtual bool ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const override;
	//~ End ITargetPlatform Interface

	ANDROIDTARGETPLATFORMCONTROLS_API virtual void InitializeDeviceDetection();

protected:

	ANDROIDTARGETPLATFORMCONTROLS_API virtual FAndroidTargetDevicePtr CreateTargetDevice(const ITargetPlatformControls& InTargetPlatform, const FString& InDeviceId, const FString& InAndroidVariant) const;

protected:
	/**
	 * Return true if this device has a supported set of extensions for this platform.
	 *
	 * @param Extensions - The GL extensions string.
	 * @param GLESVersion - The GLES version reported by this device.
	 */
	virtual bool SupportedByExtensionsString(const FString& ExtensionsString, const int GLESVersion) const
	{
		return true;
	}

	// Handles when the ticker fires.
	ANDROIDTARGETPLATFORMCONTROLS_API bool HandleTicker(float DeltaTime);

	ANDROIDTARGETPLATFORMCONTROLS_API virtual FAndroidTargetDeviceRef CreateNewDevice(const FAndroidDeviceInfo& DeviceInfo);

	// Holds a map of valid devices.
	TMap<FString, FAndroidTargetDevicePtr> Devices;

	// Holds a delegate to be invoked when the widget ticks.
	FTickerDelegate TickDelegate;

	// Handle to the registered TickDelegate.
	FTSTicker::FDelegateHandle TickDelegateHandle;

	// Pointer to the device detection handler that grabs device ids in another thread
	IAndroidDeviceDetection* DeviceDetection;

#if WITH_ENGINE

	ITargetDevicePtr DefaultDevice;
#endif //WITH_ENGINE

	FAndroidTargetPlatformSettings* AndroidTargetPlatformSettings;
};

class FAndroid_DXTTargetPlatformControls : public FAndroidTargetPlatformControls
{
public:
	FAndroid_DXTTargetPlatformControls(bool bIsClient, ITargetPlatformSettings* TargetPlatformSettings) : FAndroidTargetPlatformControls(bIsClient, TargetPlatformSettings, TEXT("DXT"))
	{
	}

	virtual bool SupportedByExtensionsString(const FString& ExtensionsString, const int GLESVersion) const override
	{
		return (ExtensionsString.Contains(TEXT("GL_NV_texture_compression_s3tc")) || ExtensionsString.Contains(TEXT("GL_EXT_texture_compression_s3tc")));
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (GConfig->GetFloat(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("TextureFormatPriority_DXT"), Priority, GEngineIni) ?
			Priority : 0.6f) * 10.0f + (IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const override;
#endif
};

class FAndroid_ASTCTargetPlatformControls : public FAndroidTargetPlatformControls
{
public:
	FAndroid_ASTCTargetPlatformControls(bool bIsClient, ITargetPlatformSettings* TargetPlatformSettings, const TCHAR* OverrideIniPlatformName = TEXT("ASTC")) 
		: FAndroidTargetPlatformControls(bIsClient, TargetPlatformSettings, OverrideIniPlatformName)
	{
	}

	virtual bool SupportedByExtensionsString(const FString& ExtensionsString, const int GLESVersion) const override
	{
		return ExtensionsString.Contains(TEXT("GL_KHR_texture_compression_astc_ldr"));
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (GConfig->GetFloat(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("TextureFormatPriority_ASTC"), Priority, GEngineIni) ?
			Priority : 0.9f) * 10.0f + (IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const;
#endif
};

// Flavorless Android platform defaults to ASTC
class FAndroid_DefaultTargetPlatformControls : public FAndroid_ASTCTargetPlatformControls
{
public:
	FAndroid_DefaultTargetPlatformControls(bool bIsClient, ITargetPlatformSettings* TargetPlatformSettings) 
		: FAndroid_ASTCTargetPlatformControls(bIsClient, TargetPlatformSettings, nullptr)
	{
	}
};

class FAndroid_ETC2TargetPlatformControls : public FAndroidTargetPlatformControls
{
public:

	FAndroid_ETC2TargetPlatformControls(bool bIsClient, ITargetPlatformSettings* TargetPlatformSettings) : FAndroidTargetPlatformControls(bIsClient, TargetPlatformSettings, TEXT("ETC2"))
	{
	}

	virtual bool SupportedByExtensionsString(const FString& ExtensionsString, const int GLESVersion) const override
	{
		return GLESVersion >= 0x30000;
	}

	virtual float GetVariantPriority() const override
	{
		float Priority;
		return (GConfig->GetFloat(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("TextureFormatPriority_ETC2"), Priority, GEngineIni) ?
			Priority : 0.2f) * 10.0f + (IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const;
#endif
};

class FAndroid_MultiTargetPlatformControls : public FAndroidTargetPlatformControls
{
	TArray<FAndroidTargetPlatformControls*> FormatTargetPlatforms;
	FString FormatTargetString;

public:
	FAndroid_MultiTargetPlatformControls(bool bIsClient, ITargetPlatformSettings* TargetPlatformSettings) : FAndroidTargetPlatformControls(bIsClient, TargetPlatformSettings, TEXT("Multi"))
	{
	}

	// set up all of the multiple formats together into this one
	void LoadFormats(TArray<FAndroidTargetPlatformControls*> SingleFormatTPs)
	{
		// sort formats by priority so higher priority formats are packaged (and thus used by the device) first
		// note that we passed this by value, not ref, so we can sort it
		SingleFormatTPs.Sort([](const FAndroidTargetPlatformControls& A, const FAndroidTargetPlatformControls& B)
			{
				float PriorityA = 0.f;
				float PriorityB = 0.f;
				FString VariantA = A.GetAndroidVariantName().Replace(TEXT("Client"), TEXT(""));
				FString VariantB = B.GetAndroidVariantName().Replace(TEXT("Client"), TEXT(""));
				GConfig->GetFloat(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), *(FString(TEXT("TextureFormatPriority_")) + VariantA), PriorityA, GEngineIni);
				GConfig->GetFloat(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), *(FString(TEXT("TextureFormatPriority_")) + VariantB), PriorityB, GEngineIni);
				return PriorityA > PriorityB;
			});

		FormatTargetPlatforms.Empty();
		FormatTargetString = TEXT("");

		TSet<FString> SeenFormats;

		// Load the TargetPlatform module for each format
		for (FAndroidTargetPlatformControls* SingleFormatTP : SingleFormatTPs)
		{
			// only use once each
			if (SeenFormats.Contains(SingleFormatTP->GetAndroidVariantName()))
			{
				continue;
			}
			SeenFormats.Add(SingleFormatTP->GetAndroidVariantName());

			bool bEnabled = false;
			FString SettingsName = FString(TEXT("bMultiTargetFormat_")) + *SingleFormatTP->GetAndroidVariantName();
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), *SettingsName, bEnabled, GEngineIni);
			if (bEnabled)
			{
				if (FormatTargetPlatforms.Num())
				{
					FormatTargetString += TEXT(",");
				}
				FormatTargetString += SingleFormatTP->GetAndroidVariantName();
				FormatTargetPlatforms.Add(SingleFormatTP);
			}
		}

		// @todo do we need this with DisplayName below?
		PlatformInfo::UpdatePlatformDisplayName(TEXT("Android_Multi"), DisplayName());
	}

	virtual float GetVariantPriority() const override
	{
		// lowest priority so specific variants are chosen first
		return (IsClientOnly() ? 0.25f : 0.5f);
	}

#if WITH_ENGINE
	virtual void GetTextureFormats(const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const;
#endif	
	virtual FText DisplayName() const override;
};
