// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AndroidTargetPlatformControls.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Android/AndroidPlatformProperties.h"
#include "MultiPlatformTargetReceiptBuildWorkers.h"
#include "IAndroidTargetPlatformControlsModule.h"
#include "IAndroidTargetPlatformSettingsModule.h"

#define LOCTEXT_NAMESPACE "FAndroidTargetPlatformControlsModule"

/**
 * Module for the Android target platform controls.
 */
class FAndroidTargetPlatformControlsModule
	: public IAndroidTargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms)
	{

	}
	/** Destructor. */
	~FAndroidTargetPlatformControlsModule()
	{
	}
protected:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName)
	{
		if (FAndroidTargetPlatformControls::IsUsable())
		{
			// ensure this is only called once
			check(SinglePlatforms.IsEmpty());

			TMap<EAndroidTextureFormatCategory, ITargetPlatformSettings*> OutMap;
			ITargetPlatformSettings* OutMulti = nullptr;
			ITargetPlatformSettings* OutXR = nullptr;
			IAndroidTargetPlatformSettingsModule* Module = FModuleManager::GetModulePtr<IAndroidTargetPlatformSettingsModule>(PlatformSettingsModuleName);
			if (Module != nullptr)
			{
				Module->GetSinglePlatformSettings(OutMap);
				OutMulti = Module->GetMultiPlatformSettings();
				OutXR = Module->GetXRPlatformSettings();
			}

			for (int32 Type = 0; Type < 2; Type++)
			{
				bool bIsClient = Type == 1;

				// flavorless must come first
				// "flavorless" (aka "vanilla") Android defaults to ASTC
				SinglePlatforms.Add(new FAndroid_DefaultTargetPlatformControls(bIsClient, OutMap[EAndroidTextureFormatCategory::Count]));
				SinglePlatforms.Add(new FAndroid_ASTCTargetPlatformControls(bIsClient, OutMap[EAndroidTextureFormatCategory::ASTC]));
				SinglePlatforms.Add(new FAndroid_DXTTargetPlatformControls(bIsClient, OutMap[EAndroidTextureFormatCategory::DXT]));
				SinglePlatforms.Add(new FAndroid_ETC2TargetPlatformControls(bIsClient, OutMap[EAndroidTextureFormatCategory::ETC2]));

				// these are used in NotifyMultiSelectedFormatsChanged, so track in another array
				MultiPlatforms.Add(new FAndroid_MultiTargetPlatformControls(bIsClient, OutMulti));
			}

			// join the single and the multi into one
			TargetPlatforms.Append(SinglePlatforms);
			TargetPlatforms.Append(MultiPlatforms);

			// set up the multi platforms now that we have all the other platforms ready to go
			NotifyMultiSelectedFormatsChanged();
		}
	}

	virtual void NotifyMultiSelectedFormatsChanged() override
	{
		for (FAndroid_MultiTargetPlatformControls* TP : MultiPlatforms)
		{
			TP->LoadFormats(SinglePlatforms);
		}
		// @todo multi needs to be passed this event!
	}

private:
	/** Holds the specific types of target platforms for NotifyMultiSelectedFormatsChanged */
	TArray<FAndroidTargetPlatformControls*> SinglePlatforms;
	TArray<FAndroid_MultiTargetPlatformControls*> MultiPlatforms;
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FAndroidTargetPlatformControlsModule, AndroidTargetPlatformControls);