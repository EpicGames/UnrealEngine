// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
IAndroid_MultiTargetPlatformModule interface 
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformControlsModule.h"

class IAndroidTargetPlatformControlsModule : public ITargetPlatformControlsModule
{
public:
	//
	// Called by AndroidRuntimeSettings to notify us when the user changes the selected texture formats for the Multi format
	//
	virtual void NotifyMultiSelectedFormatsChanged() = 0;
};
