// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidTargetPlatformSettings.h"
#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "CoreTypes.h"
#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"

FAndroidTargetPlatformSettings::FAndroidTargetPlatformSettings(const TCHAR* CookFlavor, const TCHAR* OverrideIniPlatformName)
	: TTargetPlatformSettingsBase(CookFlavor, OverrideIniPlatformName)
{
#if WITH_ENGINE
	TextureLODSettings = nullptr; // These are registered by the device profile system.
	StaticMeshLODSettings.Initialize(this);
#endif
}

// Using GetConfigSystem instead of GetPlatformValueVariable here because RenderUtils calls this too early.
bool FAndroidTargetPlatformSettings::GetIsDistanceField() const
{
	static TOptional<bool> bDistanceField;
	if (!bDistanceField.IsSet())
	{
		bDistanceField = false;
#if WITH_ENGINE
		static IConsoleVariable* CvarDistanceFields = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFields"));
		if (CvarDistanceFields != nullptr)
		{
			bDistanceField = CvarDistanceFields->GetInt() != 0;
		}
		
		GetConfigSystem()->GetBool(TEXT("/Script/Engine.RendererSettings"), TEXT("r.DistanceFields"), bDistanceField.GetValue(), GEngineIni);
#endif
	}
	return bDistanceField.GetValue();
}

int32 FAndroidTargetPlatformSettings::GetMobileShadingPath() const
{
	static TOptional<int32> MobileShadingPath;
	if (!MobileShadingPath.IsSet())
	{
		MobileShadingPath = 0;
#if WITH_ENGINE
		static IConsoleVariable* CvarShadingPath = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.ShadingPath"));
		if (CvarShadingPath != nullptr)
		{
			MobileShadingPath = CvarShadingPath->GetPlatformValueVariable(*IniPlatformName())->GetInt();
		}
#endif
	}
	return MobileShadingPath.GetValue();
}

bool FAndroidTargetPlatformSettings::GetIsMobileForwardEnableClusteredReflections() const
{
	static TOptional<bool> bMobileForwardEnableClusteredReflections;
	if (!bMobileForwardEnableClusteredReflections.IsSet())
	{
		bMobileForwardEnableClusteredReflections = false;
#if WITH_ENGINE
		static IConsoleVariable* CvarFwdEnableClusteredReflections = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.Forward.EnableClusteredReflections"));
		if (CvarFwdEnableClusteredReflections != nullptr)
		{
			bMobileForwardEnableClusteredReflections = CvarFwdEnableClusteredReflections->GetPlatformValueVariable(*IniPlatformName())->GetInt() != 0;
		}
#endif
	}
	return bMobileForwardEnableClusteredReflections.GetValue();
}

bool FAndroidTargetPlatformSettings::UsesDistanceFields() const
{
	return GetIsDistanceField();
}

bool FAndroidTargetPlatformSettings::SupportsES31() const
{
	// default no support for ES31
	bool bBuildForES31 = false;
#if WITH_ENGINE
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bBuildForES31"), bBuildForES31, GEngineIni);
#endif
	return bBuildForES31;
}

bool FAndroidTargetPlatformSettings::SupportsVulkan() const
{
	// default to not supporting Vulkan
	bool bSupportsVulkan = false;
#if WITH_ENGINE
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkan"), bSupportsVulkan, GEngineIni);
#endif
	return bSupportsVulkan;
}

bool FAndroidTargetPlatformSettings::SupportsVulkanSM5() const
{
	// default to no support for VulkanSM5
	bool bSupportsMobileVulkanSM5 = false;
#if WITH_ENGINE
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bSupportsVulkanSM5"), bSupportsMobileVulkanSM5, GEngineIni);
#endif
	return bSupportsMobileVulkanSM5;
}

bool FAndroidTargetPlatformSettings::SupportsFeature(ETargetPlatformFeatures Feature) const
{
	switch (Feature)
	{
	case ETargetPlatformFeatures::Packaging:
	case ETargetPlatformFeatures::DeviceOutputLog:
		return true;

	case ETargetPlatformFeatures::ShowAggregateDevice:
		return false;
	
	case ETargetPlatformFeatures::LowQualityLightmaps:
	case ETargetPlatformFeatures::MobileRendering:
		return SupportsES31() || SupportsVulkan();

	case ETargetPlatformFeatures::HighQualityLightmaps:
	case ETargetPlatformFeatures::DeferredRendering:
		return SupportsVulkanSM5();

	case ETargetPlatformFeatures::DistanceFieldAO:
		return UsesDistanceFields();

	case ETargetPlatformFeatures::NormalmapLAEncodingMode:
		return SupportsTextureFormatCategory(EAndroidTextureFormatCategory::ASTC);

	default:
		break;
	}

	return TTargetPlatformSettingsBase<FAndroidPlatformProperties>::SupportsFeature(Feature);
}

#if WITH_ENGINE
void FAndroidTargetPlatformSettings::GetReflectionCaptureFormats(TArray<FName>& OutFormats) const
{
	const bool bMobileDeferredShading = (GetMobileShadingPath() == 1);

	if (SupportsVulkanSM5() || bMobileDeferredShading || GetIsMobileForwardEnableClusteredReflections())
	{
		// use Full HDR with SM5 and Mobile Deferred
		OutFormats.Add(FName(TEXT("FullHDR")));
	}

	// always emit encoded
	OutFormats.Add(FName(TEXT("EncodedHDR")));
}
const UTextureLODSettings& FAndroidTargetPlatformSettings::GetTextureLODSettings() const
{
	return *TextureLODSettings;
}
#endif

void FAndroidTargetPlatformSettings::GetAllPossibleShaderFormats(TArray<FName>& OutFormats) const
{
	static FName NAME_SF_VULKAN_ES31_ANDROID(TEXT("SF_VULKAN_ES31_ANDROID"));
	static FName NAME_GLSL_ES3_1_ANDROID(TEXT("GLSL_ES3_1_ANDROID"));
	static FName NAME_SF_VULKAN_SM5_ANDROID(TEXT("SF_VULKAN_SM5_ANDROID"));

	if (SupportsVulkan())
	{
		OutFormats.AddUnique(NAME_SF_VULKAN_ES31_ANDROID);
	}

	if (SupportsVulkanSM5())
	{
		OutFormats.AddUnique(NAME_SF_VULKAN_SM5_ANDROID);
	}

	if (SupportsES31())
	{
		OutFormats.AddUnique(NAME_GLSL_ES3_1_ANDROID);
	}
}

void FAndroidTargetPlatformSettings::GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const
{
	GetAllPossibleShaderFormats(OutFormats);
}


#if WITH_ENGINE
const FStaticMeshLODSettings& FAndroidTargetPlatformSettings::GetStaticMeshLODSettings() const
{
	return StaticMeshLODSettings;
}

#endif