// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AndroidTargetPlatformSettings.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Android/AndroidPlatformProperties.h"
#include "MultiPlatformTargetReceiptBuildWorkers.h"
#include "IAndroidTargetPlatformSettingsModule.h"

/**
 * Module for the Androids target platform settings.
 */
class FAndroidTargetPlatformSettingsModule
	: public IAndroidTargetPlatformSettingsModule
{
public:

	/** Destructor. */
	~FAndroidTargetPlatformSettingsModule() 
	{
	}

public:
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms)
	{

	}
	void GetSinglePlatformSettings(TMap<EAndroidTextureFormatCategory, ITargetPlatformSettings*>& OutMap)
	{
		OutMap = SinglePlatformSettings;
	}

	ITargetPlatformSettings* GetMultiPlatformSettings()
	{
		return MultiTargetPlatformSettings;
	}

	ITargetPlatformSettings* GetXRPlatformSettings()
	{
		return XRTargetPlatformSettings;
	}

protected:
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms)
	{
		ITargetPlatformSettings* VanillaTP = new FAndroidTargetPlatformSettings();
		TargetPlatforms.Add(VanillaTP);
		SinglePlatformSettings.Add(EAndroidTextureFormatCategory::Count, VanillaTP);

		ITargetPlatformSettings* DXTTP = new FAndroid_DXTTargetPlatformSettings();
		TargetPlatforms.Add(DXTTP);
		SinglePlatformSettings.Add(EAndroidTextureFormatCategory::DXT, DXTTP);

		ITargetPlatformSettings* ETC2TP = new FAndroid_ETC2TargetPlatformSettings();
		TargetPlatforms.Add(ETC2TP);
		SinglePlatformSettings.Add(EAndroidTextureFormatCategory::ETC2, ETC2TP);

		ITargetPlatformSettings*ASTCTP = new FAndroid_ASTCTargetPlatformSettings();
		TargetPlatforms.Add(ASTCTP);
		SinglePlatformSettings.Add(EAndroidTextureFormatCategory::ASTC, ASTCTP);

		ITargetPlatformSettings* MultiTP = new FAndroid_MultiTargetPlatformSettings();
		TargetPlatforms.Add(MultiTP);
		MultiTargetPlatformSettings = MultiTP;

		ITargetPlatformSettings* OpenXRTP = new FAndroid_OpenXRTargetPlatformSettings();
		TargetPlatforms.Add(OpenXRTP);
		XRTargetPlatformSettings = OpenXRTP;
	}
private:
	ITargetPlatformSettings* MultiTargetPlatformSettings;
	ITargetPlatformSettings* XRTargetPlatformSettings;
	TMap<EAndroidTextureFormatCategory, ITargetPlatformSettings*> SinglePlatformSettings;
};

IMPLEMENT_MODULE(FAndroidTargetPlatformSettingsModule, AndroidTargetPlatformSettings);