// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Common/TargetPlatformSettingsBase.h"
#include "Android/AndroidPlatformProperties.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MonitoredProcess.h"
#include "Containers/Ticker.h"

#if WITH_ENGINE
#include "StaticMeshResources.h"
#endif // WITH_ENGINE


/** Listed in order of priority...if device supports multiple formats, first format in list will be chosen */
enum class EAndroidTextureFormatCategory
{
	DXT,
	ETC2,
	ASTC,

	Count,
};

namespace AndroidTexFormat
{
	// Compressed Texture Formats
	const static FName NameDXT1(TEXT("DXT1"));
	const static FName NameDXT5(TEXT("DXT5"));
	const static FName NameDXT5n(TEXT("DXT5n"));
	const static FName NameAutoDXT(TEXT("AutoDXT"));
	const static FName NameBC4(TEXT("BC4"));
	const static FName NameBC5(TEXT("BC5"));
	const static FName NameBC6H(TEXT("BC6H"));
	const static FName NameBC7(TEXT("BC7"));

	const static FName NameETC2_RGB(TEXT("ETC2_RGB"));
	const static FName NameETC2_RGBA(TEXT("ETC2_RGBA"));
	const static FName NameETC2_R11(TEXT("ETC2_R11"));
	const static FName NameETC2_RG11(TEXT("ETC2_RG11"));
	const static FName NameAutoETC2(TEXT("AutoETC2"));

	const static FName NameAutoASTC(TEXT("ASTC_RGBAuto"));
	const static FName NameASTC_NormalRG(TEXT("ASTC_NormalRG"));
	// L+A mode suppoprted by ARM ASTC encoder 
	const static FName NameASTC_NormalLA(TEXT("ASTC_NormalLA"));

	// Uncompressed Texture Formats
	const static FName NameBGRA8(TEXT("BGRA8"));
	const static FName NameG8(TEXT("G8"));
	const static FName NameRGBA16F(TEXT("RGBA16F"));
	const static FName NameR16F(TEXT("R16F"));
	const static FName NameG16(TEXT("G16"));

	//A1RGB555 is mapped to RGB555A1, because OpenGL GL_RGB5_A1 only supports alpha on the lowest bit
	const static FName NameA1RGB555(TEXT("A1RGB555"));
	const static FName NameRGB555A1(TEXT("RGB555A1"));

	const static FName GenericRemap[][2] =
	{
		{ NameA1RGB555,		NameRGB555A1			},
		{ NameG16,			NameR16F				}, // GLES does not support R16Unorm, fallback all Android to R16F
	};

	static const FName NameASTC_RGB_HDR(TEXT("ASTC_RGB_HDR"));
	static const FName NameASTC_RGB_LDR(TEXT("ASTC_RGB"));

	const static FName ASTCRemap[][2] =
	{
		// Default format:		ASTC format:
		{ NameDXT1,			FName(TEXT("ASTC_RGB"))		},
		{ NameDXT5,			FName(TEXT("ASTC_RGBA"))	},
		{ NameDXT5n,		FName(TEXT("ASTC_NormalAG"))},
		{ NameBC5,			NameASTC_NormalRG			},
		{ NameBC4,			NameETC2_R11				},
		{ NameBC6H,			NameASTC_RGB_HDR			},
		{ NameBC7,			FName(TEXT("ASTC_RGBA_HQ"))	},
		{ NameAutoDXT,		NameAutoASTC				},
	};

	const static FName ETCRemap[][2] =
	{
		// Default format:	ETC2 format:
		{ NameDXT1,			NameETC2_RGB	},
		{ NameDXT5,			NameETC2_RGBA	},
		{ NameDXT5n,		NameETC2_RGB	},
		{ NameBC5,			NameETC2_RG11	},
		{ NameBC4,			NameETC2_R11	},
		{ NameBC6H,			NameRGBA16F		},
		{ NameBC7,			NameETC2_RGBA	},
		{ NameAutoDXT,		NameAutoETC2	},
	};
}

class FAndroidTargetPlatformSettings : public TTargetPlatformSettingsBase<FAndroidPlatformProperties>
{
public:
	ANDROIDTARGETPLATFORMSETTINGS_API FAndroidTargetPlatformSettings(const TCHAR* CookFlavor = nullptr, const TCHAR* OverrideIniPlatformName = nullptr);

	/**
	 * Destructor
	 */
	virtual ~FAndroidTargetPlatformSettings()= default;

	virtual bool SupportsTextureFormatCategory(EAndroidTextureFormatCategory Category) const
	{
		return true;
	}

	ANDROIDTARGETPLATFORMSETTINGS_API virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const override;
	ANDROIDTARGETPLATFORMSETTINGS_API virtual void GetAllPossibleShaderFormats(TArray<FName>& OutFormats) const override;
	ANDROIDTARGETPLATFORMSETTINGS_API virtual void GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const override;

	virtual bool UsesRayTracing() const override
	{
		bool bEnableRayTracing = false;
		GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bEnableRayTracing"), bEnableRayTracing, GEngineIni);

		return bEnableRayTracing;
	}

	virtual ERayTracingRuntimeMode GetRayTracingMode() const override
	{
		return UsesRayTracing() ? ERayTracingRuntimeMode::Inline : ERayTracingRuntimeMode::Disabled;
	}

#if WITH_ENGINE
	ANDROIDTARGETPLATFORMSETTINGS_API virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override;
	ANDROIDTARGETPLATFORMSETTINGS_API virtual const UTextureLODSettings& GetTextureLODSettings() const override;
	virtual void RegisterTextureLODSettings(const UTextureLODSettings* InTextureLODSettings) override
	{
		TextureLODSettings = InTextureLODSettings;
	}
	ANDROIDTARGETPLATFORMSETTINGS_API virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings() const override;
#endif
	// query for rendering mode support
	ANDROIDTARGETPLATFORMSETTINGS_API bool SupportsES31() const;
	ANDROIDTARGETPLATFORMSETTINGS_API bool SupportsVulkan() const;
	ANDROIDTARGETPLATFORMSETTINGS_API bool SupportsVulkanSM5() const;

	virtual bool UsesDistanceFields() const override;

protected:
#if WITH_ENGINE
	// Holds a cache of the target LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds the static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;
#endif
private:
	bool GetIsDistanceField() const;
	int32 GetMobileShadingPath() const;
	bool GetIsMobileForwardEnableClusteredReflections() const;
};

class FAndroid_DXTTargetPlatformSettings : public FAndroidTargetPlatformSettings
{
public:
	FAndroid_DXTTargetPlatformSettings() : FAndroidTargetPlatformSettings(TEXT("DXT"))
	{
	}

	virtual bool SupportsTextureFormatCategory(EAndroidTextureFormatCategory Category) const override
	{
		return Category == EAndroidTextureFormatCategory::DXT;
	}
};

class FAndroid_ASTCTargetPlatformSettings : public FAndroidTargetPlatformSettings
{
public:
	FAndroid_ASTCTargetPlatformSettings(const TCHAR* Name=TEXT("ASTC")) : FAndroidTargetPlatformSettings(Name)
	{
	}

	virtual bool SupportsTextureFormatCategory(EAndroidTextureFormatCategory Category) const override
	{
		return Category == EAndroidTextureFormatCategory::ASTC;
	}
};


class FAndroid_ETC2TargetPlatformSettings : public FAndroidTargetPlatformSettings
{
public:

	FAndroid_ETC2TargetPlatformSettings() : FAndroidTargetPlatformSettings(TEXT("ETC2"))
	{
	}

	virtual bool SupportsTextureFormatCategory(EAndroidTextureFormatCategory Category) const override
	{
		return Category == EAndroidTextureFormatCategory::ETC2;
	}
};

class FAndroid_MultiTargetPlatformSettings : public FAndroidTargetPlatformSettings
{

public:
	FAndroid_MultiTargetPlatformSettings() : FAndroidTargetPlatformSettings(TEXT("Multi"))
	{
	}

	bool SupportsFeature(ETargetPlatformFeatures Feature) const override
	{
		//FAndroid_MultiTargetPlatformControls::GetTextureFormats() replaces NormalLA for NormalRG
		//So we disable the feature support to ensure shaders rebuild the blue channel correctly (See: LA_NORMALMAPS)
		if (Feature == ETargetPlatformFeatures::NormalmapLAEncodingMode)
		{
			return false;
		}
		return FAndroidTargetPlatformSettings::SupportsFeature(Feature);
	}
};

class FAndroid_OpenXRTargetPlatformSettings : public FAndroid_ASTCTargetPlatformSettings
{

public:
	FAndroid_OpenXRTargetPlatformSettings() : FAndroid_ASTCTargetPlatformSettings(TEXT("OpenXR"))
	{
	}
};
