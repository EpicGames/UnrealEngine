// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
IAndroidTargetPlatformSettingsModule interface
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "AndroidTargetPlatformSettings.h"

class IAndroidTargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetSinglePlatformSettings(TMap<EAndroidTextureFormatCategory, ITargetPlatformSettings*>& OutMap) = 0;
	virtual ITargetPlatformSettings* GetMultiPlatformSettings() = 0;
	virtual ITargetPlatformSettings* GetXRPlatformSettings() = 0;
};
