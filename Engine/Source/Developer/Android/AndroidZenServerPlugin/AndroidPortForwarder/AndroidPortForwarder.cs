// Copyright Epic Games, Inc. All Rights Reserved.

namespace AndroidZenServerPlugin;

public sealed class AndroidPortForwarder : IDisposable
{
	private CancellationTokenSource? TokenSource;
	private AndroidPortForwarderAsync? ForwarderAsync;
	private Task? StartMonitorTask;

	public void StartMonitor(string ADBPath, uint? ADBServerPortOpt, uint ZenServerPort)
	{
		Console.WriteLine("Forwarder start requested");

		TokenSource = new CancellationTokenSource();

		ForwarderAsync = new AndroidPortForwarderAsync();

		// We don't wait for startup to finish to avoid blocking current thread
		StartMonitorTask = ForwarderAsync.StartMonitorAsync(ADBPath, ADBServerPortOpt, ZenServerPort, TokenSource.Token);

		Console.WriteLine($"Forwarder started");
	}

	public void Dispose()
	{
		Console.WriteLine($"Forwarder stop requested");

		TokenSource?.Cancel();

		try
		{
			StartMonitorTask?.Wait();
		}
		catch (AggregateException exception)
		{
			// ignore token cancellation
			exception.Flatten().Handle(inner => inner is OperationCanceledException);
		}

		StartMonitorTask?.Dispose();
		ForwarderAsync?.Dispose();
		TokenSource?.Dispose();

		TokenSource = null;
		ForwarderAsync = null;
		StartMonitorTask = null;

		Console.WriteLine($"Forwarder stopped");
	}
}
