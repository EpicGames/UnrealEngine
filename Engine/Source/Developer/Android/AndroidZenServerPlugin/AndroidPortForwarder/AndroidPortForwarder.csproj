<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <!--
    <OutputType>Exe</OutputType>
    <DefineConstants>TEST_CLI</DefineConstants>
    -->
    <TargetFramework>net10.0</TargetFramework>
    <ImplicitUsings>enable</ImplicitUsings>
    <Nullable>enable</Nullable>
    <PublishAot>true</PublishAot>
    <InvariantGlobalization>true</InvariantGlobalization>
    <IsAotCompatible>true</IsAotCompatible>
    <StripSymbols>false</StripSymbols>
    <Optimize>true</Optimize>
    <OptimizationPreference>Size</OptimizationPreference>
    <RootNamespace>AndroidZenServerPlugin</RootNamespace>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
  </PropertyGroup>
  <ItemGroup>
    <PackageReference Include="AdvancedSharpAdbClient" Version="3.4.14" />
  </ItemGroup>
  <ItemGroup>
    <NativeLibrary Include="$(SolutionDir)$(Platform)\$(Configuration)\ZenServerAdapter.lib" Condition="$(RuntimeIdentifier.StartsWith('win'))" />
    <NativeLibrary Include="$(SolutionDir)$(Platform)\$(Configuration)\ZenServerAdapter.a" Condition="!$(RuntimeIdentifier.StartsWith('win'))" />
    <LinkerArg Include="/INCLUDE:GetTransportPluginVersion" Condition="$(RuntimeIdentifier.StartsWith('win'))" />
    <LinkerArg Include="/INCLUDE:CreateTransportPlugin" Condition="$(RuntimeIdentifier.StartsWith('win'))" />
  </ItemGroup>
</Project>
