// Copyright Epic Games, Inc. All Rights Reserved.

namespace AndroidZenServerPlugin;

using AdvancedSharpAdbClient;
using AdvancedSharpAdbClient.DeviceCommands;
using AdvancedSharpAdbClient.Models;
using System.Diagnostics;
using System.Net;

public sealed class AndroidPortForwarderAsync : IDisposable
{
	private AdbClient? Client;
	private DeviceMonitor? DeviceMonitor;

	public async Task StartMonitorAsync(string ADBPath, uint? ADBServerPortOpt, uint ZenServerPort, CancellationToken CancellationToken)
	{
		try
		{
			if (string.IsNullOrEmpty(ADBPath))
			{
				ADBPath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
			}

			uint ResolvedADBServerPort = ADBServerPortOpt.GetValueOrDefault((uint)AdbClient.AdbServerPort);
			Client = new AdbClient(new IPEndPoint(IPAddress.Loopback, (int)ResolvedADBServerPort));

			// inject custom command line client
			AdbServer.Instance = new AdbServer(Client, Factories.AdbSocketFactory, Path => new DetachedAdbCommandLineClient(Path));

			await AdbServer.Instance.StartServerAsync(ADBPath, false, CancellationToken);

			Console.WriteLine($"Forwarder server started at '{ADBPath}'");

			DeviceMonitor = new DeviceMonitor(Client.CreateAdbSocket());

			DeviceMonitor.DeviceConnected += async (_, args) =>
			{
				if (args.Device.State == DeviceState.Online)
				{
					await ForwardPortsToDeviceAsync(Client, args.Device, ZenServerPort, CancellationToken);
				}
			};
			DeviceMonitor.DeviceChanged += async (_, args) =>
			{
				if (args.Device.State == DeviceState.Online)
				{
					await ForwardPortsToDeviceAsync(Client, args.Device, ZenServerPort, CancellationToken);
				}
			};

			await DeviceMonitor.StartAsync(CancellationToken);

			Console.WriteLine($"Forwarder ready");
		}
		catch (System.Net.Sockets.SocketException e)
		{
			Console.WriteLine($"Forwarder failed to connect to adb due to '{e}'");
		}
		catch (FileNotFoundException e)
		{
			Console.WriteLine($"Forwarder failed to connect to adb due to '{e}'");
		}
		catch (Exception e)
		{
			Console.Error.WriteLine($"Forwarder failed to start due to '{e}'");
		}
	}

	public void Dispose()
	{
		try
		{
			DeviceMonitor?.Dispose();
		}
		catch (Exception e)
		{
			Console.Error.WriteLine($"Forwarder failed to dispose due to '{e}'");
		}
		finally
		{
			DeviceMonitor = null;
			Client = null;
		}
	}

	private static async Task ForwardPortsToDeviceAsync(IAdbClient Client, DeviceData Device, uint Port, CancellationToken CancellationToken)
	{
		try
		{
			if (await DoesDeviceNeedsZenStreamingAsync(Client, Device, CancellationToken))
			{
				string Forward = $"tcp:{Port}";
				await Client.CreateReverseForwardAsync(Device, Forward, Forward, true, CancellationToken);

				Console.WriteLine($"Android device '{Device.Serial}' '{Forward}' reverse forwarded.");
			}
			else
			{
				Console.WriteLine($"Android device '{Device.Serial}' has no zen enabled apps installed, skipping.");
			}
		}
		catch (Exception e)
		{
			Console.Error.WriteLine($"Forwarder failed to forward ports due to '{e}'");
		}
	}

	private static async Task<bool> DoesDeviceNeedsZenStreamingAsync(IAdbClient Client, DeviceData Device, CancellationToken CancellationToken)
	{
		// We quickly determinate if any apps on a device needs streaming via
		// probing for a special activity added to every app's AndroidManifest.xml that uses zen streaming.
		const string TokenActivity = "com.epicgames.unreal.Zen.StreamingEnabled";

		const string Cmd = $"pm query-activities --brief -a {TokenActivity}";

		bool bAnyActivities = false;

		await Client.ExecuteShellCommandAsync(Device, Cmd, output =>
		{
			// Output will contain "No activities found" or "%number% activities found:".
			// The strings are hardcoded in frameworks/base/services/core/java/com/android/server/pm/PackageManagerShellCommand.java, see
			// https://cs.android.com/android/platform/superproject/+/android15-qpr2-release:frameworks/base/services/core/java/com/android/server/pm/PackageManagerShellCommand.java;l=1442-1445
			if (output.Contains("activities found"))
			{
				bAnyActivities = !output.Contains("No activities found");
				return false;
			}

			return true;
		}, CancellationToken);

		return bAnyActivities;
	}

	// Right now launching adb with bInheritHandles makes adb server inherit zen server port handles,
	// which prevents them from being released when zen server is shutdown/restarted.
	// To circumvent this we launch adb commands via UseShellExecute.
	// See https://github.com/dotnet/runtime/issues/13943
	internal class DetachedAdbCommandLineClient(string AdbPath) : AdbCommandLineClient(AdbPath)
	{
		private bool ShouldRunAsDetached(string Filename, string Command, ICollection<string>? ErrorOutput, ICollection<string>? StandardOutput)
		{
			// always run start-server in detached mode without inheriting handles
			return Command.EndsWith("start-server") ||
			       (ErrorOutput == null && StandardOutput == null);
		}

		protected override int RunProcess(string Filename, string Command, ICollection<string>? ErrorOutput, ICollection<string>? StandardOutput, int Timeout)
		{
			if (!ShouldRunAsDetached(Filename, Command, ErrorOutput, StandardOutput))
			{
				return base.RunProcess(Filename, Command, ErrorOutput, StandardOutput, Timeout);
			}

			ProcessStartInfo Info = new()
			{
				FileName = Filename,
				Arguments = Command,
				WindowStyle = ProcessWindowStyle.Hidden,
				UseShellExecute = true,
			};

			using Process Handle = Process.Start(Info) ?? throw new Exception($"Failed to start process '{Filename}' with command '{Command}'");

			if (!Handle.WaitForExit(Timeout))
			{
				Handle.Kill();
			}

			return Handle.ExitCode;
		}

		protected override async Task<int> RunProcessAsync(string Filename, string Command, ICollection<string>? ErrorOutput, ICollection<string>? StandardOutput, CancellationToken CancellationToken = default)
		{
			if (!ShouldRunAsDetached(Filename, Command, ErrorOutput, StandardOutput))
			{
				return await base.RunProcessAsync(Filename, Command, ErrorOutput, StandardOutput, CancellationToken);
			}

			ProcessStartInfo Info = new()
			{
				FileName = Filename,
				Arguments = Command,
				WindowStyle = ProcessWindowStyle.Hidden,
				UseShellExecute = true,
			};

			using Process Handle = Process.Start(Info) ?? throw new Exception($"Failed to start process '{Filename}' with command '{Command}'");

			try
			{
				await Handle.WaitForExitAsync(CancellationToken);
			}
			catch (OperationCanceledException)
			{
				if (!Handle.HasExited)
				{
					Handle.Kill();
				}

				throw;
			}

			return Handle.ExitCode;
		}
	}
}
