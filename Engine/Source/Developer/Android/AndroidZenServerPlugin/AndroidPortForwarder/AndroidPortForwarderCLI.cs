// Copyright Epic Games, Inc. All Rights Reserved.

#if TEST_CLI

using AndroidZenServerPlugin;

TextWriter OriginalOut = Console.Out;
using ConsoleRedirector Redirector = new ConsoleRedirector((Level, Line) => OriginalOut.WriteLine($"[{Level}] {Line}"));

Console.WriteLine("Android Port Forwarder CLI testbed");

ManualResetEvent exitEvent = new ManualResetEvent(false);

Console.CancelKeyPress += (_, eventArgs) =>
{
	Console.WriteLine("Exit requested");
	eventArgs.Cancel = true;
	exitEvent.Set();
};

using AndroidPortForwarder forwarder = new AndroidPortForwarder();

forwarder.StartMonitor(string.Empty, null, 8558);

exitEvent.WaitOne();

#endif