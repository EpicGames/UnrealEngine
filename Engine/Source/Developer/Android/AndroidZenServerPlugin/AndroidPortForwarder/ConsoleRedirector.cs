// Copyright Epic Games, Inc. All Rights Reserved.

namespace AndroidZenServerPlugin;

// Redirects console output, errors and unhandled exceptions to a delegate invoked per output line.
internal sealed class ConsoleRedirector : IDisposable
{
	public enum LogLevel
	{
		Info,
		Error
	};

	public delegate void OutputDelegate(LogLevel Level, string Line);

	private TextWriter OriginalOut;
	private TextWriter OriginalError;

	public ConsoleRedirector(OutputDelegate Delegate)
	{
		OriginalOut = Console.Out;
		OriginalError = Console.Error;

		Console.SetOut(new DelegateWriter(Delegate, LogLevel.Info));
		Console.SetError(new DelegateWriter(Delegate, LogLevel.Error));

		AppDomain.CurrentDomain.UnhandledException += OnCurrentDomainOnUnhandledException;
	}

	public void Dispose()
	{
		AppDomain.CurrentDomain.UnhandledException -= OnCurrentDomainOnUnhandledException;

		Console.SetOut(OriginalOut);
		Console.SetError(OriginalError);
	}

	private void OnCurrentDomainOnUnhandledException(object _, UnhandledExceptionEventArgs args)
	{
		Console.Error.WriteLine($"Unhandled exception: {args.ExceptionObject}");
		Console.Error.Flush();
	}

	private sealed class DelegateWriter : StringWriter
	{
		private readonly LogLevel LogLevel;
		private readonly OutputDelegate Delegate;

		public DelegateWriter(OutputDelegate InDelegate, LogLevel InLogLevel)
		{
			LogLevel = InLogLevel;
			Delegate = InDelegate;
		}

		public override void Flush()
		{
			using var Reader = new StringReader(ToString());
			GetStringBuilder().Clear();

			string? Line;
			while ((Line = Reader.ReadLine()) != null)
			{
				Delegate(LogLevel, Line);
			}
		}

		private void FlushLine()
		{
			Delegate(LogLevel, ToString());
			GetStringBuilder().Clear();
		}

		public override void Write(char[] buffer, int index, int count)
		{
			if (buffer == CoreNewLine && index == 0 && count == CoreNewLine.Length)
			{
				FlushLine();
			}
			else
			{
				base.Write(buffer, index, count);
			}
		}

		public override void Write(string? value)
		{
			if (value == NewLine)
			{
				FlushLine();
			}
			else
			{
				base.Write(value);
			}
		}
	}
}