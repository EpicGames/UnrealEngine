// Copyright Epic Games, Inc. All Rights Reserved.

namespace AndroidZenServerPlugin;

using System.Runtime.InteropServices;

public static class ZenServerAdapter
{
	private static ConsoleRedirector? ConsoleRedirector;
	private static AndroidPortForwarder? AndroidPortForwarder;

	[UnmanagedCallersOnly(EntryPoint = "StartAndroidPortForwarder")]
	private static unsafe void StartAndroidPortForwarder(delegate*<void*, int, char*, void> LogCallback, void* UsrPtr, IntPtr ADBPathPtr, uint* ADBServerPortOptPtr, uint ZenServerPort)
	{
		ConsoleRedirector?.Dispose();
		ConsoleRedirector = new ConsoleRedirector((Level, Line) =>
		{
			IntPtr LinePtr = Marshal.StringToHGlobalAnsi(Line);
			try
			{
				LogCallback(UsrPtr, LogLevelToInt(Level), (char*)LinePtr);
			}
			finally
			{
				Marshal.FreeHGlobal(LinePtr);
			}
		});

		string ADBPath = ADBPathPtr != IntPtr.Zero ? (Marshal.PtrToStringAnsi(ADBPathPtr) ?? string.Empty) : string.Empty;
		uint? ADBServerPortOpt = ADBServerPortOptPtr != null ? *ADBServerPortOptPtr : null;

		AndroidPortForwarder?.Dispose();
		AndroidPortForwarder = new AndroidPortForwarder();

		AndroidPortForwarder.StartMonitor(ADBPath, ADBServerPortOpt, ZenServerPort);
	}

	[UnmanagedCallersOnly(EntryPoint = "StopAndroidPortForwarder")]
	private static void StopAndroidPortForwarder()
	{
		AndroidPortForwarder?.Dispose();
		AndroidPortForwarder = null;

		ConsoleRedirector?.Dispose();
		ConsoleRedirector = null;
	}

	private static int LogLevelToInt(ConsoleRedirector.LogLevel Level)
	{
		return Level switch
		{
			ConsoleRedirector.LogLevel.Info => 0,
			ConsoleRedirector.LogLevel.Error => 1,
			_ => 0
		};
	}
}
