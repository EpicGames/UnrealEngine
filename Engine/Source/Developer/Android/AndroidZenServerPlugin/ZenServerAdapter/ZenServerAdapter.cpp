// Copyright Epic Games, Inc. All Rights Reserved.

#include <transportplugin.h>
#include <atomic>
#include <cstdarg>
#include <optional>
#include <string>
#include <vector>

static const uint32_t kPluginVersion = 3;
static const uint32_t kDefaultZenServerPort = 8558;

// native -> managed API
extern "C"
{
	typedef void (*LogCallback)(void* UsrPtr, int32_t Level, const char* Buffer);

	void StartAndroidPortForwarder(LogCallback Callback, void* UsrPtr, const char* ADBPath, const uint32_t* ADBServerPortOpt, uint32_t ZenServerPort);
	void StopAndroidPortForwarder();
}

// logging utility
static void
#if __clang__
__attribute__((format(printf, 2, 3)))
#endif
LogMessage(zen::TransportLogger* Logger, const zen::TransportLogger::LogLevel Level, const char* Format, ...)
{
	char Buffer[1024];

	va_list Args;
	va_start(Args, Format);
	const int RequiredSize = vsnprintf(Buffer, sizeof(Buffer), Format, Args);
	if (RequiredSize < sizeof(Buffer))
	{
		Logger->LogMessage(Level, Buffer);
	}
	else
	{
		char* AllocBuffer = (char*)malloc(RequiredSize + 1);
		vsnprintf(AllocBuffer, RequiredSize + 1, Format, Args);
		Logger->LogMessage(Level, AllocBuffer);
		free(AllocBuffer);
	}
	va_end(Args);
}

#define LOG_INFO(Logger, ...) LogMessage(Logger, zen::TransportLogger::LogLevel::Info, __VA_ARGS__)
#define LOG_WARN(Logger, ...) LogMessage(Logger, zen::TransportLogger::LogLevel::Warn, __VA_ARGS__)
#define LOG_ERR(Logger, ...)  LogMessage(Logger, zen::TransportLogger::LogLevel::Err,  __VA_ARGS__)

class ZenServerAdapter : public zen::TransportPlugin
{
public:
	ZenServerAdapter(zen::TransportLogger* InLogger)
		: Logger(InLogger)
	{
	}
	virtual ~ZenServerAdapter() = default;

	uint32_t AddRef() const override
	{
		return const_cast<ZenServerAdapter*>(this)->ReferenceCount.fetch_add(1) + 1;
	}

	uint32_t Release() const override
	{
		const uint32_t RefCount = const_cast<ZenServerAdapter*>(this)->ReferenceCount.fetch_sub(1);
		if (RefCount <= 1)
		{
			delete this;
		}
		return RefCount - 1;
	}

	void Configure(const char* OptionTag, const char* OptionValue) override
	{
		if (!strcmp(OptionTag, "port"))
		{
			if (!ParseOption(OptionValue, ZenServerPort))
			{
				LOG_ERR(Logger, "Can't parse zen server port value '%s'", OptionValue);
			}
		}
		else if (!strcmp(OptionTag, "adb_server_port"))
		{
			uint32_t ServerPortTemp = 0;
			if (ParseOption(OptionValue, ServerPortTemp))
			{
				ADBServerPort = ServerPortTemp;
			}
			else
			{
				LOG_ERR(Logger, "Can't parse adb server port value '%s'", OptionValue);
			}
		}
		else if (!strcmp(OptionTag, "adb_path"))
		{
			ADBPath = std::string(OptionValue);
		}
	}

	void Initialize(zen::TransportServer* ServerInterface) override
	{
		uint32_t ADBServerPortTemp = ADBServerPort.value_or(0);
		const uint32_t* ADBServerPortTempPtr = ADBServerPort.has_value() ? &ADBServerPortTemp : nullptr;

		StartAndroidPortForwarder(
		[](void* ThisPtr, int32_t Level, const char* Buffer)
		{
			static_cast<ZenServerAdapter*>(ThisPtr)->Logger->LogMessage(Level == 1 ? zen::TransportLogger::LogLevel::Err : zen::TransportLogger::LogLevel::Info, Buffer);
		},
		this, ADBPath.has_value() ? ADBPath.value().c_str() : nullptr, ADBServerPortTempPtr, ZenServerPort);
	}

	void Shutdown() override
	{
		StopAndroidPortForwarder();
	}

	const char* GetDebugName() override
	{
		return "AndroidPortForwarder";
	}

	bool IsAvailable() override
	{
		return true;
	}

private:
	std::atomic<uint32_t> ReferenceCount = {0};
	uint32_t ZenServerPort = kDefaultZenServerPort;
	std::optional<uint32_t> ADBServerPort = std::optional<uint32_t>();
	std::optional<std::string> ADBPath = std::optional<std::string>();
	zen::TransportLogger* Logger = nullptr;

	static bool ParseOption(const char* Option, uint32_t& Result)
	{
		char* EndPtr = nullptr;
		const unsigned long Value = std::strtoul(Option, &EndPtr, 10);

		if (Option == EndPtr || Value > UINT32_MAX)
		{
			return false;
		}

		Result = static_cast<uint32_t>(Value);
		return true;
	}
};

#if defined(_MSC_VER)
#	define DLL_TRANSPORT_API __declspec(dllexport)
#else
#	define DLL_TRANSPORT_API
#endif

extern "C" DLL_TRANSPORT_API void GetTransportPluginVersion(uint32_t* OutApiVersion, uint32_t* OutPluginVersion)
{
	if (OutApiVersion != nullptr)
	{
		*OutApiVersion = zen::kTransportApiVersion;
	}

	if (OutPluginVersion != nullptr)
	{
		*OutPluginVersion = kPluginVersion;
	}
}

extern "C" DLL_TRANSPORT_API zen::TransportPlugin* CreateTransportPlugin(zen::TransportLogger* Logger)
{
	return new ZenServerAdapter(Logger);
}

#if TEST_CLI

extern "C"
{
	void StartAndroidPortForwarder(LogCallback Callback, void* UsrPtr, const char* ADBServerPortOpt, const uint32_t* ADBServerPortOpt, uint32_t ZenServerPort) {}
	void StopAndroidPortForwarder() {}
}

class StubTransportLogger : public zen::TransportLogger
{
public:
	void LogMessage(LogLevel Level, const char* Message) override { printf("%u %s\n", (uint8_t)Level, Message); }
};

int main()
{
	auto Plugin = CreateTransportPlugin(new StubTransportLogger);
	Plugin->Configure("port", "abc");
	Plugin->Initialize(nullptr);
	while (true)
	{
		Sleep(100);
	}
	return 0;
}

#endif