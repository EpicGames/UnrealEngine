Zen server plugin for Android device support
Reverse-forwards Zen Server port to any connected android device

Use `dotnet publish -r win-x64` to build the .dll